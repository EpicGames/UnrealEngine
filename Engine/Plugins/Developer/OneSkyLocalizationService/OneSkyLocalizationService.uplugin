{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.5",
	"FriendlyName" : "OneSky",
	"Description" : "OneSky localization service",
	"Category" : "Localization",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "OneSkyLocalizationService",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}