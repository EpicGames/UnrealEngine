// Copyright Epic Games, Inc. All Rights Reserved.

#include "OneSkyConnection.h"
#include "ILocalizationServiceModule.h"
#include "OneSkyLocalizationServiceModule.h"
#include "OneSkyLocalizationServiceCommand.h"

#define LOCTEXT_NAMESPACE "OneSkyConnection"

FOneSkyConnection::FOneSkyConnection(const FOneSkyConnectionInfo& InConnectionInfo)
:	bEstablishedConnection(false)
{
	EstablishConnection(InConnectionInfo);
}

FOneSkyConnection::~FOneSkyConnection()
{
	Disconnect();
}

bool FOneSkyConnection::IsValidConnection()
{
	return bEstablishedConnection;
}

void FOneSkyConnection::Disconnect()
{
	// TODO: Maybe do something
}

bool FOneSkyConnection::EstablishConnection(const FOneSkyConnectionInfo& InConnectionInfo)
{
	//Connection assumed successful
	bEstablishedConnection = true;

	UE_LOGF(LogLocalizationService, Verbose, "OneSky connection created: %ls", *InConnectionInfo.Name);
	
	return bEstablishedConnection;
}

FScopedOneSkyConnection::FScopedOneSkyConnection( const class FOneSkyLocalizationServiceCommand& InCommand )
	: Connection(NULL)
	, Concurrency(InCommand.Concurrency)
{
	Initialize(InCommand.ConnectionInfo);
}

FScopedOneSkyConnection::FScopedOneSkyConnection( ELocalizationServiceOperationConcurrency::Type InConcurrency, const FOneSkyConnectionInfo& InConnectionInfo )
	: Connection(NULL)
	, Concurrency(InConcurrency)
{
	Initialize(InConnectionInfo);
}

void FScopedOneSkyConnection::Initialize( const FOneSkyConnectionInfo& InConnectionInfo )
{
	if (Concurrency == ELocalizationServiceOperationConcurrency::Synchronous)
	{
		// Synchronous commands reuse the same persistent connection to reduce
		// the number of expensive connection attempts.
		FOneSkyLocalizationServiceModule& OneSkyLocalizationService = FModuleManager::LoadModuleChecked<FOneSkyLocalizationServiceModule>("OneSkyLocalizationService");
		if ( OneSkyLocalizationService.GetProvider().EstablishPersistentConnection() )
		{
			Connection = OneSkyLocalizationService.GetProvider().GetPersistentConnection();
		}
	}
	else
	{
		// Async commands form a new connection for each attempt because
		// using the persistent connection is not threadsafe
		FOneSkyConnection* NewConnection = new FOneSkyConnection(InConnectionInfo);
		if ( NewConnection->IsValidConnection() )
		{
			Connection = NewConnection;
		}
	}
}

FScopedOneSkyConnection::~FScopedOneSkyConnection()
{
	if(Connection != NULL)
	{
		if (Concurrency == ELocalizationServiceOperationConcurrency::Asynchronous)
		{
			// Remove this connection as it is only temporary
			Connection->Disconnect();
			delete Connection;
		}
		
		Connection = NULL;
	}
}

#undef LOCTEXT_NAMESPACE
