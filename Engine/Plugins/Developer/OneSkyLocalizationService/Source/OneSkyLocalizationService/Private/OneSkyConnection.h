// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILocalizationServiceProvider.h"

struct FOneSkyConnectionInfo;

class FOneSkyConnection
{
public:
	

	FOneSkyConnection(const FOneSkyConnectionInfo& InConnectionInfo);
	/** API Specific close of localization service project*/
	~FOneSkyConnection();


	/** Returns true if connection is currently active */
	bool IsValidConnection();

	/** If the connection is valid, disconnect from the server */
	void Disconnect();

	/**
	 * Make a valid connection if possible
	 * @param InConnectionInfo		Connection credentials
	 */
	bool EstablishConnection(const FOneSkyConnectionInfo& InConnectionInfo);

public:

	/** true if connection was successfully established */
	bool bEstablishedConnection;
};

/**
 * Connection that is used within specific scope
 */
class FScopedOneSkyConnection
{
public:
	/** 
	 * Constructor - establish a connection.
	 * The concurrency of the passed in command determines whether the persistent connection is 
	 * used or another connection is established (connections cannot safely be used across different
	 * threads).
	 */
	FScopedOneSkyConnection( const class FOneSkyLocalizationServiceCommand& InCommand );

	/** 
	 * Constructor - establish a connection.
	 * The concurrency passed in determines whether the persistent connection is used or another 
	 * connection is established (connections cannot safely be used across different threads).
	 */
	FScopedOneSkyConnection( ELocalizationServiceOperationConcurrency::Type InConcurrency, const FOneSkyConnectionInfo& InConnectionInfo );

	/**
	 * Destructor - disconnect if this is a temporary connection
	 */
	~FScopedOneSkyConnection();

	/** Get the connection wrapped by this class */
	FOneSkyConnection& GetConnection() const
	{
		return *Connection;
	}

	/** Check if this connection is valid */
	bool IsValid() const
	{
		return Connection != NULL;
	}

private:
	/** Set up the connection */
	void Initialize( const FOneSkyConnectionInfo& InConnectionInfo );

private:
	/** The OneSky connection we are using */
	FOneSkyConnection* Connection;

	/** The concurrency of this connection */
	ELocalizationServiceOperationConcurrency::Type Concurrency;
};
