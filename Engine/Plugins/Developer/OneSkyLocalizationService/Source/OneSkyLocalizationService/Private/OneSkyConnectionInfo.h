// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

struct FScriptContainerElement;

/** Helper struct defining parameters we need to establish a connection */
struct FOneSkyConnectionInfo
{
	/** Name of this connection configuration */
	FString Name;

	/** OneSky ApiKey */
	FString ApiKey;

	/** OneSky ApiSecret */
	FString ApiSecret;
};
