// Copyright Epic Games, Inc. All Rights Reserved.

#include "OneSkyLocalizationServiceCommand.h"
#include "OneSkyLocalizationServiceModule.h"

FOneSkyLocalizationServiceCommand::FOneSkyLocalizationServiceCommand(const TSharedRef<class ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation, const TSharedRef<class IOneSkyLocalizationServiceWorker, ESPMode::ThreadSafe>& InWorker, const FLocalizationServiceOperationComplete& InOperationCompleteDelegate /*= FTranslationServiceOperationComplete() */)
	: Operation(InOperation)
	, Worker(InWorker)
	, OperationCompleteDelegate(InOperationCompleteDelegate)
	, bExecuteProcessed(0)
	, bCancelled(0)
	, bCommandSuccessful(false)
	, bConnectionDropped(false)
	, bAutoDelete(true)
	, Concurrency(ELocalizationServiceOperationConcurrency::Synchronous)
{
	// grab the providers settings here, so we don't access them once the worker thread is launched
	check(IsInGameThread());
	FOneSkyLocalizationServiceModule& OneSkyLocalizationService = FModuleManager::LoadModuleChecked<FOneSkyLocalizationServiceModule>( "OneSkyLocalizationService" );
	ConnectionInfo = OneSkyLocalizationService.AccessSettings().GetConnectionInfo();
}

bool FOneSkyLocalizationServiceCommand::DoWork()
{
	bCommandSuccessful = Worker->Execute(*this);

	// This is not set here so we can wait for the worker's HTTP response delegate to be called 
	// FPlatformAtomics::InterlockedExchange(&bExecuteProcessed, 1);

	while (!bExecuteProcessed)
	{
		FPlatformProcess::Sleep(0.01f);

		// If the editor was closed, our callbacks won't get called anyway, so just kill the thread
		if (IsEngineExitRequested())
		{
			Abandon();
		}
	}

	return bCommandSuccessful;
}

void FOneSkyLocalizationServiceCommand::Abandon()
{
	FPlatformAtomics::InterlockedExchange(&bExecuteProcessed, 1);
}

void FOneSkyLocalizationServiceCommand::DoThreadedWork()
{
	Concurrency = ELocalizationServiceOperationConcurrency::Asynchronous;
	DoWork();
}

void FOneSkyLocalizationServiceCommand::Cancel()
{
	FPlatformAtomics::InterlockedExchange(&bCancelled, 1);
}

bool FOneSkyLocalizationServiceCommand::IsCanceled() const
{
	return bCancelled != 0;
}
