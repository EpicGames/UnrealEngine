// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILocalizationServiceProvider.h"
#include "OneSkyConnectionInfo.h"
#include "Misc/IQueuedWork.h"

DECLARE_DELEGATE_RetVal(bool, FOnIsCancelled);

/**
 * Used to execute OneSky commands multi-threaded.
 */
class FOneSkyLocalizationServiceCommand : public IQueuedWork
{
public:

	FOneSkyLocalizationServiceCommand(const TSharedRef<class ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation, const TSharedRef<class IOneSkyLocalizationServiceWorker, ESPMode::ThreadSafe>& InWorker, const FLocalizationServiceOperationComplete& InOperationCompleteDelegate = FLocalizationServiceOperationComplete() );

	/**
	 * This is where the real thread work is done. All work that is done for
	 * this queued object should be done from within the call to this function.
	 */
	bool DoWork();

	/**
	 * Tells the queued work that it is being abandoned so that it can do
	 * per object clean up as needed. This will only be called if it is being
	 * abandoned before completion. NOTE: This requires the object to delete
	 * itself using whatever heap it was allocated in.
	 */
	virtual void Abandon() override;

	/**
	 * This method is also used to tell the object to cleanup but not before
	 * the object has finished it's work.
	 */ 
	virtual void DoThreadedWork() override;

	/** Attempt to cancel the operation */
	void Cancel();

	/** Is the operation canceled? */
	bool IsCanceled() const;

public:
	/** Connection parameters, reproduced here because if is not safe to access the provider's settings from another thread */
	FOneSkyConnectionInfo ConnectionInfo;

	/** Operation we want to perform - contains outward-facing parameters & results */
	TSharedRef<class ILocalizationServiceOperation, ESPMode::ThreadSafe> Operation;

	/** The object that will actually do the work */
	TSharedRef<class IOneSkyLocalizationServiceWorker, ESPMode::ThreadSafe> Worker;

	/** Delegate to notify when this operation completes */
	FLocalizationServiceOperationComplete OperationCompleteDelegate;

	/**If true, this command has been processed by the Localization service thread*/
	volatile int32 bExecuteProcessed;

	/**If true, this command has been cancelled*/
	volatile int32 bCancelled;

	/**If true, the Localization service command succeeded*/
	bool bCommandSuccessful;

	/**If true, the Localization service connection was dropped while this command was being executed*/
	bool bConnectionDropped;

	/** If true, this command will be automatically cleaned up in Tick() */
	bool bAutoDelete;

	/** Whether we are running multi-treaded or not*/
	ELocalizationServiceOperationConcurrency::Type Concurrency;

	/** The GUID of the Localization Target we are working with */
	FGuid TargetGuid;

	/** Files to perform this operation on */
	TArray< FString > Files;

	/** Translations to perform this operation on */
	TArray< FLocalizationServiceTranslationIdentifier > Translations;

	/**Info and/or warning message message storage*/
	TArray< FText > InfoMessages;

	/**Potential error message storage*/
	TArray< FText > ErrorMessages;
};
