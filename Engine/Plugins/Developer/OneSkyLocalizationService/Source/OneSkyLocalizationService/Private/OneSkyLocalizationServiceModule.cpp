// Copyright Epic Games, Inc. All Rights Reserved.

#include "OneSkyLocalizationServiceModule.h"
#include "Misc/App.h"
#include "OneSkyLocalizationServiceOperations.h"
#include "Features/IModularFeatures.h"

#define LOCTEXT_NAMESPACE "OneSkyTranslationService"

template<typename Type>
static TSharedRef<IOneSkyLocalizationServiceWorker, ESPMode::ThreadSafe> CreateWorker()
{
	return MakeShareable( new Type() );
}

void FOneSkyLocalizationServiceModule::StartupModule()
{
	// Register our operations

	// ProjectGroup API
	OneSkyLocalizationServiceProvider.RegisterWorker("ListProjectGroups", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyListProjectGroupsWorker>));
	OneSkyLocalizationServiceProvider.RegisterWorker("ShowProjectGroup", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyShowProjectGroupWorker>));
	OneSkyLocalizationServiceProvider.RegisterWorker("CreateProjectGroup", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyCreateProjectGroupWorker>));
	OneSkyLocalizationServiceProvider.RegisterWorker("ListProjectGroupLanguages", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyListProjectGroupLanguagesWorker>));

	// Project API
	OneSkyLocalizationServiceProvider.RegisterWorker("ListProjectsInGroup", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyListProjectsInGroupWorker>));
	OneSkyLocalizationServiceProvider.RegisterWorker("ShowProject", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyShowProjectWorker>));
	OneSkyLocalizationServiceProvider.RegisterWorker("CreateProject", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyCreateProjectWorker>));
	OneSkyLocalizationServiceProvider.RegisterWorker("ListProjectLanguages", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyListProjectLanguagesWorker>));

	// Translation API
	OneSkyLocalizationServiceProvider.RegisterWorker("TranslationStatus", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyTranslationStatusWorker>));
	OneSkyLocalizationServiceProvider.RegisterWorker("DownloadLocalizationTargetFile", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyTranslationExportWorker>));

	// Files API
	OneSkyLocalizationServiceProvider.RegisterWorker("ListUploadedFiles", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyListUploadedFilesWorker>));
	OneSkyLocalizationServiceProvider.RegisterWorker("UploadLocalizationTargetFile", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyUploadFileWorker>));

	// Import API
	OneSkyLocalizationServiceProvider.RegisterWorker("ShowImportTask", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyShowImportTaskWorker>));

	// Phrase Collections API
	OneSkyLocalizationServiceProvider.RegisterWorker("ListPhraseCollections", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyListPhraseCollectionsWorker>));
	//OneSkyLocalizationServiceProvider.RegisterWorker("ShowPhraseCollection", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyShowPhraseCollectionWorker>));
	//OneSkyLocalizationServiceProvider.RegisterWorker("ImportPhraseCollections", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyImportPhraseCollectionsWorker>));

	// ProjectTypes API
	//OneSkyLocalizationServiceProvider.RegisterWorker("ListProjectTypes", FGetOneSkyLocalizationServiceWorker::CreateStatic(&CreateWorker<FOneSkyListProjectTypesWorker>));

	// load our settings
	OneSkyLocalizationServiceSettings.LoadSettings();

	// Bind our Localization service provider to the editor
	IModularFeatures::Get().RegisterModularFeature( "LocalizationService", &OneSkyLocalizationServiceProvider );
}

void FOneSkyLocalizationServiceModule::ShutdownModule()
{
	// shut down the provider, as this module is going away
	OneSkyLocalizationServiceProvider.Close();

	// unbind provider from editor
	IModularFeatures::Get().UnregisterModularFeature( "LocalizationService", &OneSkyLocalizationServiceProvider );
}

FOneSkyLocalizationServiceSettings& FOneSkyLocalizationServiceModule::AccessSettings()
{
	return OneSkyLocalizationServiceSettings;
}

void FOneSkyLocalizationServiceModule::SaveSettings()
{
	if (FApp::IsUnattended() || IsRunningCommandlet())
	{
		return;
	}

	OneSkyLocalizationServiceSettings.SaveSettings();
}

IMPLEMENT_MODULE(FOneSkyLocalizationServiceModule, OneSkyLocalizationService);

#undef LOCTEXT_NAMESPACE
