// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "OneSkyLocalizationServiceProvider.h"
#include "OneSkyLocalizationServiceSettings.h"

class FOneSkyLocalizationServiceModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Access the OneSky Localization service settings */
	FOneSkyLocalizationServiceSettings& AccessSettings();

	/** Save the OneSky Localization service settings */
	void SaveSettings();

	/** Access the one and only Perforce provider */
	FOneSkyLocalizationServiceProvider& GetProvider()
	{
		return OneSkyLocalizationServiceProvider;
	}
	
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FOneSkyLocalizationServiceModule& Get()
	{
		return FModuleManager::LoadModuleChecked< FOneSkyLocalizationServiceModule >("OneSkyLocalizationService");
	}

private:
	/** The one and only OneSky Localization service provider */
	FOneSkyLocalizationServiceProvider OneSkyLocalizationServiceProvider;

	/** The settings for OneSky Localization service */
	FOneSkyLocalizationServiceSettings OneSkyLocalizationServiceSettings;
};
