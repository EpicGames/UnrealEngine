// Copyright Epic Games, Inc. All Rights Reserved.

#include "OneSkyLocalizationServiceOperations.h"
#include "Interfaces/IHttpRequest.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "ILocalizationServiceModule.h"
#include "OneSkyLocalizationServiceState.h"
#include "OneSkyLocalizationServiceModule.h"
#include "OneSkyLocalizationServiceCommand.h"
#include "Interfaces/IHttpResponse.h"
#include "HttpModule.h"
#include "PlatformHttp.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "StructDeserializer.h"
#include "Logging/MessageLog.h"

#define LOCTEXT_NAMESPACE "OneSkyLocalizationService"

FString GetAuthenticationParameters(const FOneSkyConnectionInfo& ConnectionInfo)
{
	//https://github.com/onesky/api-documentation-platform/blob/master/README.md#authentication

	FString UtcNowUnixTimeStamp = FString::FromInt(FDateTime::UtcNow().ToUnixTimestamp());
	FString Hash = FMD5::HashAnsiString(*(UtcNowUnixTimeStamp + ConnectionInfo.ApiSecret));

	return "api_key=" + ConnectionInfo.ApiKey + "&dev_hash=" + Hash + "&timestamp=" + UtcNowUnixTimeStamp;
}

FString AddAuthenticationParameters(const FOneSkyConnectionInfo& ConnectionInfo, const FString& Url)
{
	return Url + "?" + GetAuthenticationParameters(ConnectionInfo);
}

static bool UpdateCachedLocalizationStates(const TMap<FLocalizationServiceTranslationIdentifier, TSharedRef<FOneSkyLocalizationServiceState, ESPMode::ThreadSafe>, FDefaultSetAllocator, FLocalizationServiceTranslationIdentifierKeyFuncs<TSharedRef<FOneSkyLocalizationServiceState, ESPMode::ThreadSafe>>>& InResults)
{
	FOneSkyLocalizationServiceModule& OneSkyLocalizationService = FOneSkyLocalizationServiceModule::Get();
	for (auto It = InResults.CreateConstIterator(); It; ++It)
	{
		TSharedRef<FOneSkyLocalizationServiceState, ESPMode::ThreadSafe> State = OneSkyLocalizationService.GetProvider().GetStateInternal(It.Key());
		State->SetState(It.Value()->GetState());
		State->SetTranslation(It.Value()->GetTranslationString());
		State->TimeStamp = FDateTime::Now();
	}

	return InResults.Num() > 0;
}

static FString GetFileFormat(FString FileExtension)
{
	if (FileExtension == ".po")
	{
		return "GNU_PO";
	}
	else if (FileExtension == ".pot")
	{
		return "GNU_POT";
	}
	else
	{
		return "";
	}
}

static bool DeserializeResponseToStruct(void* OutStruct, UStruct& TypeInfo, FHttpResponsePtr HttpResponse)
{
	bool bResult = false;
	FString ResponseStr = HttpResponse->GetContentAsString();
	FText ErrorText;

	if (HttpResponse.IsValid())
	{
		if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{

			// Write string to FMemoryWriter to force unicode
			TArray<uint8> InBytes;
			FMemoryWriter Writer(InBytes);
			Writer.SetForceUnicode(true);
			Writer << ResponseStr;

			FMemoryReader Reader(InBytes);
			// FMemoryWriter writes size of string at beginning, need to ignore this or json parsing errors occur
			Reader.Seek(4); 

			FJsonStructDeserializerBackend Backend(Reader);

			bResult = FStructDeserializer::Deserialize(OutStruct, TypeInfo, Backend);
		}
		else
		{
			ErrorText = FText::Format(LOCTEXT("InvalidResponse", "Invalid response. code={0} error={1}"), FText::FromString(FString::FromInt(HttpResponse->GetResponseCode())), FText::FromString(ResponseStr));
		}
	}

	if (!bResult)
	{
		UE_LOGF(LogLocalizationService, Warning, "%ls", *(ErrorText.ToString()));
	}

	return bResult;
}

FName FOneSkyConnect::GetName() const
{
	return "FOneSkyConnect";
}

bool FOneSkyConnect::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	//FOneSkyLocalizationServiceModule& OneSkyLocalizationService = FModuleManager::LoadModuleChecked<FOneSkyLocalizationServiceModule>("OneSkyLocalizationService");
	//OneSkyLocalizationService.GetProvider().RetrieveProjectGroups();

	return true;
}


// LIST PROJECT GROUPS

bool FOneSkyListProjectGroupsWorker::Execute(FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;
	TSharedPtr<FOneSkyListProjectGroupsOperation, ESPMode::ThreadSafe> ListProjectGroupsOp = StaticCastSharedRef<FOneSkyListProjectGroupsOperation>(InCommand.Operation);
	
	int32 InStartPage = -1;
	int32 InItemsPerPage = -1;
	
	if (ListProjectGroupsOp.IsValid())
	{
		InStartPage = ListProjectGroupsOp->GetInStartPage();
		InItemsPerPage = ListProjectGroupsOp->GetInItemsPerPage();
	}

	FString Url = AddAuthenticationParameters(InCommand.ConnectionInfo, "https://platform.api.onesky.io/1/project-groups");
	if (InStartPage != -1 && InItemsPerPage != -1)
	{
		Url += FString(TEXT("&page=")) +FString::FromInt(InStartPage) + FString(TEXT("&per_page=")) + FString::FromInt(InItemsPerPage);
	}

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyListProjectGroupsWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}



void FOneSkyListProjectGroupsWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
			UScriptStruct& StructType = *(FOneSkyListProjectGroupsResponse::StaticStruct());
			bResult = DeserializeResponseToStruct(&OutListProjectGroupsResponse, StructType, HttpResponse);
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

FName FOneSkyListProjectGroupsWorker::GetName() const
{
	return "ListProjectGroupsWorker";
}

// end LIST PROJECT GROUPS


// SHOW PROJECT GROUP

FName FOneSkyShowProjectGroupWorker::GetName() const
{
	return "ShowProjectGroupWorker";
}

bool FOneSkyShowProjectGroupWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;

	TSharedPtr<FOneSkyShowProjectGroupOperation, ESPMode::ThreadSafe> ShowProjectGroupOp = StaticCastSharedRef<FOneSkyShowProjectGroupOperation>(InCommand.Operation);

	int32 InProjectGroupId = -1;

	if (ShowProjectGroupOp.IsValid())
	{
		InProjectGroupId = ShowProjectGroupOp->GetInProjectGroupId();
	}

	FString Url = "https://platform.api.onesky.io/1/project-groups/";
	Url += FString::FromInt(InProjectGroupId);
	Url = AddAuthenticationParameters(InCommand.ConnectionInfo, Url);

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyShowProjectGroupWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}

void FOneSkyShowProjectGroupWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		UScriptStruct& StructType = *(FOneSkyShowProjectGroupResponse::StaticStruct());
		bResult = DeserializeResponseToStruct(&OutShowProjectGroupResponse, StructType, HttpResponse);
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

// end SHOW PROJECT GROUP


// CREATE PROJECT GROUP

FName FOneSkyCreateProjectGroupWorker::GetName() const
{
	return "CreateProjectGroupWorker";
}

bool FOneSkyCreateProjectGroupWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;

	TSharedPtr<FOneSkyCreateProjectGroupOperation, ESPMode::ThreadSafe> CreateProjectGroupOp = StaticCastSharedRef<FOneSkyCreateProjectGroupOperation>(InCommand.Operation);

	FString InProjectGroupName = "";
	FString InProjectGroupBaseLocale = "";

	if (CreateProjectGroupOp.IsValid())
	{
		//we need to url encode for special characters (especially other languages)
		InProjectGroupName = FPlatformHttp::UrlEncode(CreateProjectGroupOp->GetInProjectGroupName());
		InProjectGroupBaseLocale = FPlatformHttp::UrlEncode(CreateProjectGroupOp->GetInProjectGroupBaseLocale());
	}

	FString Url = "https://platform.api.onesky.io/1/project-groups";
	FString Parameters = GetAuthenticationParameters(InCommand.ConnectionInfo) + FString(TEXT("&name=")) + InProjectGroupName + FString(TEXT("&locale=")) + InProjectGroupBaseLocale;
	Url += "?" + Parameters;

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// Seems to work fine with no content payload (same info is in the URL anyway)
	// Leaving code here as a reference for later
	// Write string to FMemoryWriter to force unicode
	//TArray<uint8> InBytes;
	//FMemoryWriter Writer(InBytes);
	//Writer.SetForceUnicode(true);
	//Writer << Parameters;
	//FMemoryReader Reader(InBytes);
	//// FMemoryWriter writes size of string at beginning, need to ignore this or parsing errors occur
	//InBytes.RemoveAt(3);
	//InBytes.RemoveAt(2);
	//InBytes.RemoveAt(1);
	//InBytes.RemoveAt(0);
	//HttpRequest->SetContent(InBytes);

	
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyCreateProjectGroupWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);

	// UTF-8
	// Not sure if these matter if we don't send a content payload
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("text/html; charset=utf-8"));

	HttpRequest->SetVerb(TEXT("POST"));
	// kick off http request to read 
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}

void FOneSkyCreateProjectGroupWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		UScriptStruct& StructType = *(FOneSkyCreateProjectGroupResponse::StaticStruct());
		bResult = DeserializeResponseToStruct(&OutCreateProjectGroupResponse, StructType, HttpResponse);
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

// end CREATE PROJECT GROUP


// LIST PROJECT GROUP LANGUAGES

FName FOneSkyListProjectGroupLanguagesWorker::GetName() const
{
	return "ListProjectGroupLanguagesWorker";
}

bool FOneSkyListProjectGroupLanguagesWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;

	TSharedPtr<FOneSkyListProjectGroupLanguagesOperation, ESPMode::ThreadSafe> ListProjectGroupLanguagesOp = StaticCastSharedRef<FOneSkyListProjectGroupLanguagesOperation>(InCommand.Operation);

	int32 InProjectGroupId = -1;

	if (ListProjectGroupLanguagesOp.IsValid())
	{
		InProjectGroupId = ListProjectGroupLanguagesOp->GetInProjectGroupId();
	}

	FString Url = "https://platform.api.onesky.io/1/project-groups/" + FString::FromInt(InProjectGroupId) / "languages";
	Url = AddAuthenticationParameters(InCommand.ConnectionInfo, Url);

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyListProjectGroupLanguagesWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}

void FOneSkyListProjectGroupLanguagesWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		UScriptStruct& StructType = *(FOneSkyListProjectGroupLanguagesResponse::StaticStruct());
		bResult = DeserializeResponseToStruct(&OutListProjectGroupLanguagesResponse, StructType, HttpResponse);
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

// end LIST PROJECT GROUP LANGUAGES



// LIST PROJECTS IN GROUP

FName FOneSkyListProjectsInGroupWorker::GetName() const
{
	return FName(TEXT("FOneSkyListProjectsInGroupWorker"));
}

bool FOneSkyListProjectsInGroupWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;
	TSharedPtr<FOneSkyListProjectsInGroupOperation, ESPMode::ThreadSafe> ListProjectsInGroupOp = StaticCastSharedRef<FOneSkyListProjectsInGroupOperation>(InCommand.Operation);

	int32 InProjectGroupId = -1;

	if (ListProjectsInGroupOp.IsValid())
	{
		InProjectGroupId = ListProjectsInGroupOp->GetInProjectGroupId();
	}

	FString Url = "https://platform.api.onesky.io/1/project-groups/" + FString::FromInt(InProjectGroupId) / "projects";
	Url = AddAuthenticationParameters(InCommand.ConnectionInfo, Url);

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyListProjectsInGroupWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}

void FOneSkyListProjectsInGroupWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		UScriptStruct& StructType = *(FOneSkyListProjectsInGroupResponse::StaticStruct());
		bResult = DeserializeResponseToStruct(&OutListProjectsInGroupResponse, StructType, HttpResponse);
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

// end LIST PROJECTS IN GROUP


// SHOW PROJECT

FName FOneSkyShowProjectWorker::GetName() const
{
	return FName(TEXT("FOneSkyShowProjectWorker"));
}

bool FOneSkyShowProjectWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;

	TSharedPtr<FOneSkyShowProjectOperation, ESPMode::ThreadSafe> ShowProjectOp = StaticCastSharedRef<FOneSkyShowProjectOperation>(InCommand.Operation);

	int32 InProjectId = -1;

	if (ShowProjectOp.IsValid())
	{
		InProjectId = ShowProjectOp->GetInProjectId();
	}

	FString Url = "https://platform.api.onesky.io/1/projects/" + FString::FromInt(InProjectId);
	Url = AddAuthenticationParameters(InCommand.ConnectionInfo, Url);

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyShowProjectWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}

void FOneSkyShowProjectWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		UScriptStruct& StructType = *(FOneSkyShowProjectResponse::StaticStruct());
		bResult = DeserializeResponseToStruct(&OutShowProjectResponse, StructType, HttpResponse);
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

// end SHOW PROJECT


// CREATE PROJECT

FName FOneSkyCreateProjectWorker::GetName() const
{
	return FName(TEXT("FOneSkyCreateProjectWorker"));
}

bool FOneSkyCreateProjectWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;

	TSharedPtr<FOneSkyCreateProjectOperation, ESPMode::ThreadSafe> CreateProjectOp = StaticCastSharedRef<FOneSkyCreateProjectOperation>(InCommand.Operation);

	FString InProjectGroupName = "";
	FString InProjectDescription = "";
	FString InProjectType = "";
	int32 InProjectGroupId = -1;

	if (CreateProjectOp.IsValid())
	{
		//we need to url encode for special characters (especially other languages)
		InProjectGroupName = FPlatformHttp::UrlEncode(CreateProjectOp->GetInProjectGroupName());
		InProjectDescription = FPlatformHttp::UrlEncode(CreateProjectOp->GetInProjectDescription());
		InProjectType = FPlatformHttp::UrlEncode(CreateProjectOp->GetInProjectType());
		InProjectGroupId = CreateProjectOp->GetInProjectGroupId();
	}

	FString Url = "https://platform.api.onesky.io/1/project-groups/" + FString::FromInt(InProjectGroupId) + "/projects";
	FString Parameters = GetAuthenticationParameters(InCommand.ConnectionInfo) + FString(TEXT("&name=")) + InProjectGroupName + FString(TEXT("&description=")) + InProjectDescription + FString(TEXT("&project_type=") + InProjectType);
	Url += "?" + Parameters;

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read 
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyCreateProjectWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("text/html; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("POST"));
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}

void FOneSkyCreateProjectWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		UScriptStruct& StructType = *(FOneSkyCreateProjectResponse::StaticStruct());
		bResult = DeserializeResponseToStruct(&OutCreateProjectResponse, StructType, HttpResponse);
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

// end CREATE PROJECT


// LIST PROJECT LANGUAGES

FName FOneSkyListProjectLanguagesWorker::GetName() const
{
	return "ListProjectLanguagesWorker";
}

bool FOneSkyListProjectLanguagesWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;

	TSharedPtr<FOneSkyListProjectLanguagesOperation, ESPMode::ThreadSafe> ListProjectLanguagesOp = StaticCastSharedRef<FOneSkyListProjectLanguagesOperation>(InCommand.Operation);

	int32 InProjectId = -1;

	if (ListProjectLanguagesOp.IsValid())
	{
		InProjectId = ListProjectLanguagesOp->GetInProjectId();
	}

	FString Url = "https://platform.api.onesky.io/1/projects/" + FString::FromInt(InProjectId) / "languages";
	Url = AddAuthenticationParameters(InCommand.ConnectionInfo, Url);

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyListProjectLanguagesWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}

void FOneSkyListProjectLanguagesWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		UScriptStruct& StructType = *(FOneSkyListProjectLanguagesResponse::StaticStruct());
		bResult = DeserializeResponseToStruct(&OutListProjectLanguagesResponse, StructType, HttpResponse);
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

//end LIST PROJECT LANGUAGES


// TRANSLATION STATUS

FName FOneSkyTranslationStatusWorker::GetName() const
{
	return "TranslationStatusWorker";
}

bool FOneSkyTranslationStatusWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;
	TSharedPtr<FOneSkyTranslationStatusOperation, ESPMode::ThreadSafe> TranslationStatusOp = StaticCastSharedRef<FOneSkyTranslationStatusOperation>(InCommand.Operation);

	int32 InProjectId = -1;
	FString InFileName = "";
	FString InLocale = "";

	if (TranslationStatusOp.IsValid())
	{
		InProjectId = TranslationStatusOp->GetInProjectId();
		InFileName = FPlatformHttp::UrlEncode(TranslationStatusOp->GetInFileName());
		InLocale = FPlatformHttp::UrlEncode(TranslationStatusOp->GetInLocale());
	}
	
	FString Url = AddAuthenticationParameters(InCommand.ConnectionInfo, "https://platform.api.onesky.io/1/projects/" + FString::FromInt(InProjectId) / "translations/status");
	Url += FString(TEXT("&file_name=")) + InFileName + FString(TEXT("&locale=")) + InLocale;

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyTranslationStatusWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}

void FOneSkyTranslationStatusWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		UScriptStruct& StructType = *(FOneSkyTranslationStatusResponse::StaticStruct());
		bResult = DeserializeResponseToStruct(&OutTranslationStatusResponse, StructType, HttpResponse);
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

// end TRANSLATION STATUS


// TRANSLATION EXPORT

FName FOneSkyTranslationExportWorker::GetName() const
{
	return "TranslationExportWorker";
}

bool FOneSkyTranslationExportWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;
	TSharedPtr<FDownloadLocalizationTargetFile, ESPMode::ThreadSafe> DownloadLocTargetOp = StaticCastSharedRef<FDownloadLocalizationTargetFile>(InCommand.Operation);

	FGuid InTargetGuid;
	FString InLocale;
	FString InRelativeOutputFilePathAndName;

	if (DownloadLocTargetOp.IsValid())
	{
		InTargetGuid = DownloadLocTargetOp->GetInTargetGuid();
		InLocale = FPlatformHttp::UrlEncode(DownloadLocTargetOp->GetInLocale());
		InRelativeOutputFilePathAndName = DownloadLocTargetOp->GetInRelativeOutputFilePathAndName();
	}

	int32 InProjectId = -1;
	FString InSourceFileName = "";
	FString InExportFileName = "";

	// OneSky project settings are accessed by their localization target guid. These settings are set in the Localization Dashboard
	FOneSkyLocalizationTargetSetting* Settings = FOneSkyLocalizationServiceModule::Get().AccessSettings().GetSettingsForTarget(InTargetGuid);
	if (Settings != nullptr)
	{
		ILocalizationServiceProvider& Provider = ILocalizationServiceModule::Get().GetProvider();
		InProjectId = Settings->OneSkyProjectId;
		InSourceFileName = FPlatformHttp::UrlEncode(Settings->OneSkyFileName);
		InExportFileName = FPlatformHttp::UrlEncode(FPaths::ConvertRelativePathToFull(InRelativeOutputFilePathAndName));
	}

	FString Url = AddAuthenticationParameters(InCommand.ConnectionInfo, "https://platform.api.onesky.io/1/projects/" + FString::FromInt(InProjectId) / "translations");
	Url += FString(TEXT("&source_file_name=")) + InSourceFileName + FString(TEXT("&locale=")) + InLocale;
	if (!InExportFileName.IsEmpty())
	{
		Url += "&export_file_name=" + InExportFileName;
	}

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyTranslationExportWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}

void FOneSkyTranslationExportWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		FString ResponseStr = HttpResponse->GetContentAsString();
		FText ErrorText;

		if (HttpResponse.IsValid())
		{
			EHttpResponseCodes::Type ResponseCode = (EHttpResponseCodes::Type) HttpResponse->GetResponseCode();

			if (EHttpResponseCodes::IsOk(ResponseCode))
			{
				// TODO: Test Accepted case.  Haven't been able to make this happen yet...
				if (ResponseCode == EHttpResponseCodes::Accepted)
				{
					if (Command != nullptr)
					{
						TSharedPtr<FDownloadLocalizationTargetFile, ESPMode::ThreadSafe> TranslationExportOp = StaticCastSharedRef<FDownloadLocalizationTargetFile>(Command->Operation);
						if (TranslationExportOp.IsValid())
						{
							// The file is not ready, try again
							TSharedRef<FDownloadLocalizationTargetFile, ESPMode::ThreadSafe> NewTranslationExportLanguagesOp = ILocalizationServiceOperation::Create<FDownloadLocalizationTargetFile>();
							NewTranslationExportLanguagesOp->SetInTargetGuid(TranslationExportOp->GetInTargetGuid());
							NewTranslationExportLanguagesOp->SetInLocale(TranslationExportOp->GetInLocale());
							NewTranslationExportLanguagesOp->SetInRelativeOutputFilePathAndName(TranslationExportOp->GetInRelativeOutputFilePathAndName());
							// TODO: Can't spawn a new Worker here, as it tries to access the OneSky Connection Info from a thread that is not the main thread.
							// Instead spawn in callback?
							//ILocalizationServiceModule::Get().GetProvider().Execute(NewTranslationExportLanguagesOp, TArray<FLocalizationServiceTranslationIdentifier>(), ELocalizationServiceOperationConcurrency::Asynchronous);
							// For now, error
							ErrorText = LOCTEXT("TranslationExportQueryFailedRetryNotImplemented", "Translation Export Query Failed: Retry not yet implemented.");
							bResult = false;
						}
						else
						{
							ErrorText = LOCTEXT("TranslationExportQueryFailedTranslationExportOpInvalid", "Translation Export Query Failed: Translation Export Operation is invalid.");
						}

						return;
					}
					else
					{
						ErrorText = LOCTEXT("TranslationExportQueryFailedCommandNull", "Translation Export Query Failed: Command is null.");
					}
				}
				else if (ResponseCode == EHttpResponseCodes::NoContent)
				{
					bResult = false;
				}
				// If there's no response code, then this is the file
				else
				{
					if (Command != nullptr)
					{
						TSharedPtr<FDownloadLocalizationTargetFile, ESPMode::ThreadSafe> TranslationExportOp = StaticCastSharedRef<FDownloadLocalizationTargetFile>(Command->Operation);
						if (TranslationExportOp.IsValid())
						{
							// Path is relative to game directory
							FString Filename = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir() / TranslationExportOp->GetInRelativeOutputFilePathAndName());

							if (Filename.IsEmpty())
							{
								ErrorText = LOCTEXT("InvalidExportFilename", "Export filename is invalid");
								bResult = false;
							}
							else if (FFileHelper::SaveStringToFile(ResponseStr, *Filename, FFileHelper::EEncodingOptions::ForceUnicode))
							{
								bResult = true;
							}
							else
							{
								ErrorText = LOCTEXT("FailedToWriteFile", "Could not write file.");
								bResult = false;
							}
						}
						else
						{
							ErrorText = LOCTEXT("ExportFilenameNotFound", "Could not find export file name.");
							bResult = false;
						}
					}
				}
			}
			else
			{
				ErrorText = FText::Format(LOCTEXT("InvalidResponse", "Invalid response. code={0} error={1}"), FText::FromString(FString::FromInt(HttpResponse->GetResponseCode())), FText::FromString(ResponseStr));
			}
		}

		if (!bResult)
		{
			UE_LOGF(LogLocalizationService, Warning, "%ls", *(ErrorText.ToString()));
			if (Command != nullptr)
			{
				Command->ErrorMessages.Add(ErrorText);
				TSharedPtr<FDownloadLocalizationTargetFile, ESPMode::ThreadSafe> DownloadLocTargetOp = StaticCastSharedRef<FDownloadLocalizationTargetFile>(Command->Operation);
				if (DownloadLocTargetOp.IsValid())
				{
					DownloadLocTargetOp->SetOutErrorText(ErrorText);
				}
			}
		}
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

// end TRANSLATION EXPORT


// LIST UPLOADED FILES

FName FOneSkyListUploadedFilesWorker::GetName() const
{
	return "ListUploadedFilesWorker";
}

bool FOneSkyListUploadedFilesWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;
	TSharedPtr<FOneSkyListUploadedFilesOperation, ESPMode::ThreadSafe> ListUploadedFilesOp = StaticCastSharedRef<FOneSkyListUploadedFilesOperation>(InCommand.Operation);

	int32 InProjectId = -1;
	int32 InStartPage = -1;
	int32 InItemsPerPage = -1;

	if (ListUploadedFilesOp.IsValid())
	{
		InProjectId = ListUploadedFilesOp->GetInProjectId();
		InStartPage = ListUploadedFilesOp->GetInStartPage();
		InItemsPerPage = ListUploadedFilesOp->GetInItemsPerPage();
	}

	FString Url = AddAuthenticationParameters(InCommand.ConnectionInfo, "https://platform.api.onesky.io/1/projects/" + FString::FromInt(InProjectId) / "files");
	if (InStartPage != -1 && InItemsPerPage != -1)
	{
		Url += FString(TEXT("&page=")) + FString::FromInt(InStartPage) + FString(TEXT("&per_page=")) + FString::FromInt(InItemsPerPage);
	}

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyListUploadedFilesWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}

void FOneSkyListUploadedFilesWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		UScriptStruct& StructType = *(FOneSkyListUploadedFilesResponse::StaticStruct());
		bResult = DeserializeResponseToStruct(&OutListUploadedFilesResponse, StructType, HttpResponse);
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

// end LIST UPLOADED FILES


// UPLOAD FILE

FName FOneSkyUploadFileWorker::GetName() const
{
	return "UploadFileWorker";
}

bool FOneSkyUploadFileWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;
	TSharedPtr<FUploadLocalizationTargetFile, ESPMode::ThreadSafe> UploadFileOp = StaticCastSharedRef<FUploadLocalizationTargetFile>(InCommand.Operation);

	FGuid InTargetGuid;
	FString InFilePathAndName = "";
	FString InLocale = "";
	bool bInIsKeepingAllStrings = true;
	FString InFileFormat = "";

	if (UploadFileOp.IsValid())
	{
		InTargetGuid = UploadFileOp->GetInTargetGuid();
		InLocale = UploadFileOp->GetInLocale();
		// Path is relative to game directory
		InFilePathAndName = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir() / UploadFileOp->GetInRelativeInputFilePathAndName());
		bInIsKeepingAllStrings = UploadFileOp->GetPreserveAllText();
		InFileFormat = GetFileFormat(FPaths::GetExtension(InFilePathAndName, true));
	}

	int32 InProjectId = -1;
	FString InOneSkyTargetFileName = "";

	// OneSky project settings are accessed by their localization target guid. These settings are set in the Localization Dashboard
	FOneSkyLocalizationTargetSetting* Settings = FOneSkyLocalizationServiceModule::Get().AccessSettings().GetSettingsForTarget(InTargetGuid);
	if (Settings != nullptr)
	{
		InProjectId = Settings->OneSkyProjectId;
		InOneSkyTargetFileName = FPlatformHttp::UrlEncode(Settings->OneSkyFileName);
	}

	FString Url = AddAuthenticationParameters(InCommand.ConnectionInfo, "https://platform.api.onesky.io/1/projects/" + FString::FromInt(InProjectId) / "files");
	Url += "&file_format=" + InFileFormat + "&locale=" + InLocale;
	if (!bInIsKeepingAllStrings)
	{
		Url += "&is_keeping_all_strings=false";
	}

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyUploadFileWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	FString Boundary = "---------------------------" + FString::FromInt(FDateTime::Now().GetTicks());
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("multipart/form-data; boundary =" + Boundary));
	HttpRequest->SetVerb(TEXT("POST"));

	FString FileContents;
	if (FFileHelper::LoadFileToString(FileContents, *InFilePathAndName))
	{
		// Format the file the way OneSky wants it (in a form, with the file contents being a field called "file")
		FString PrefixBoundry = "\r\n--" + Boundary + "\r\n";
		FString FileHeader = "Content-Disposition: form-data; name=\"file\"; filename=\"" + InOneSkyTargetFileName + "\"\r\nContent-Type: " + InFileFormat + "\r\n\r\n";
		FString SuffixBoundary = "\r\n--" + Boundary + "--\r\n";
		FString ContentsString = PrefixBoundry + FileHeader + FileContents + SuffixBoundary;

		HttpRequest->SetContentAsString(ContentsString);
		HttpRequest->ProcessRequest();
	}
	else
	{
		Command->bCommandSuccessful = false;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}

	return InCommand.bCommandSuccessful;
}

void FOneSkyUploadFileWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		UScriptStruct& StructType = *(FOneSkyUploadFileResponse::StaticStruct());
		bResult = DeserializeResponseToStruct(&OutUploadFileResponse, StructType, HttpResponse);
	}

	if (bResult)
	{
		TSharedPtr<FUploadLocalizationTargetFile, ESPMode::ThreadSafe> UploadFileOp = StaticCastSharedRef<FUploadLocalizationTargetFile>(Command->Operation); //-V595

		FGuid InTargetGuid;
		int32 InProjectId = -1;

		if (UploadFileOp.IsValid())
		{
			InTargetGuid = UploadFileOp->GetInTargetGuid();

			// OneSky project settings are accessed by their localization target guid. These settings are set in the Localization Dashboard
			FOneSkyLocalizationTargetSetting* Settings = FOneSkyLocalizationServiceModule::Get().AccessSettings().GetSettingsForTarget(InTargetGuid);
			if (Settings != nullptr)
			{
				InProjectId = Settings->OneSkyProjectId;
			}
		}

		FShowImportTaskQueueItem ImportTaskQueueItem;
		ImportTaskQueueItem.ImportId = OutUploadFileResponse.data.import.id;
		ImportTaskQueueItem.ProjectId = InProjectId;
		// Wait one minute before querying the status of the import
		ImportTaskQueueItem.ExecutionDelayInSeconds = 60.0f;
		ImportTaskQueueItem.CreatedTimestamp = FDateTime::UtcNow();
		FOneSkyLocalizationServiceModule::Get().GetProvider().EnqueShowImportTask(ImportTaskQueueItem);
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

// end UPLOAD FILE


// LIST PHRASE COLLECTIONS

FName FOneSkyListPhraseCollectionsWorker::GetName() const
{
	return "ListPhraseCollections";
}

bool FOneSkyListPhraseCollectionsWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;
	TSharedPtr<FOneSkyListPhraseCollectionsOperation, ESPMode::ThreadSafe> ListPhraseCollectionsOp = StaticCastSharedRef<FOneSkyListPhraseCollectionsOperation>(InCommand.Operation);

	int32 InProjectId = -1;
	int32 InPage = -1;
	int32 InItemsPerPage = -1;

	if (ListPhraseCollectionsOp.IsValid())
	{
		InProjectId = ListPhraseCollectionsOp->GetInProjectId();
		InPage = ListPhraseCollectionsOp->GetInPage();
		InItemsPerPage = ListPhraseCollectionsOp->GetInItemsPerPage();
	}

	FString Url = AddAuthenticationParameters(InCommand.ConnectionInfo, "https://platform.api.onesky.io/1/projects/" + FString::FromInt(InProjectId) / "phrase-collections");
	if (InPage >= 1)
	{
		Url += FString(TEXT("&page=")) + FString::FromInt(InPage);
	}
	if (InItemsPerPage >= 1)
	{
		Url += FString(TEXT("&per_page=")) + FString::FromInt(InItemsPerPage);
	}

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyListPhraseCollectionsWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}

void FOneSkyListPhraseCollectionsWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		UScriptStruct& StructType = *(FOneSkyListPhraseCollectionsResponse::StaticStruct());
		bResult = DeserializeResponseToStruct(&OutListPhraseCollectionsResponse, StructType, HttpResponse);
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

// end LIST PHRASE COLLECTIONS


// SHOW IMPORT TASK

FName FOneSkyShowImportTaskWorker::GetName() const
{
	return "ShowImportTask";
}

bool FOneSkyShowImportTaskWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
{
	// Store pointer to command so we can access it in the http request callback
	Command = &InCommand;

	TSharedPtr<FOneSkyShowImportTaskOperation, ESPMode::ThreadSafe> ShowImportTaskOp = StaticCastSharedRef<FOneSkyShowImportTaskOperation>(InCommand.Operation);

	int32 InProjectId = -1;
	int32 InImportId = -1;
	int32 InExecutionDelayInSeconds = -1;
	FDateTime InCreationTimeStamp = FDateTime(1970, 1, 1);

	if (ShowImportTaskOp.IsValid())
	{
		InProjectId = ShowImportTaskOp->GetInProjectId();
		InImportId = ShowImportTaskOp->GetInImportId();
		InExecutionDelayInSeconds = ShowImportTaskOp->GetInExecutionDelayInSeconds();
		InCreationTimeStamp = ShowImportTaskOp->GetInCreationTimestamp();
	}

	double ElapsedTimeInSeconds = (FDateTime::UtcNow() - InCreationTimeStamp).GetTotalSeconds();

	while (!IsEngineExitRequested() && (ElapsedTimeInSeconds <= InExecutionDelayInSeconds))
	{
		ElapsedTimeInSeconds = (FDateTime::UtcNow() - InCreationTimeStamp).GetTotalSeconds();
		FPlatformProcess::Sleep(0.05f);
	}

	FString Url = "https://platform.api.onesky.io/1/projects/" + FString::FromInt(InProjectId) / "import-tasks/" + FString::FromInt(InImportId);
	Url = AddAuthenticationParameters(InCommand.ConnectionInfo, Url);

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	// kick off http request to read
	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FOneSkyShowImportTaskWorker::Query_HttpRequestComplete);
	HttpRequest->SetURL(Url);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();

	return InCommand.bCommandSuccessful;
}

void FOneSkyShowImportTaskWorker::Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bool bResult = false;

	if (bSucceeded)
	{
		UScriptStruct& StructType = *(FOneSkyShowImportTaskResponse::StaticStruct());
		bResult = DeserializeResponseToStruct(&OutShowImportTaskResponse, StructType, HttpResponse);
	}

	if (bResult)
	{
		TSharedPtr<FOneSkyShowImportTaskOperation, ESPMode::ThreadSafe> ShowImportTaskOp = StaticCastSharedRef<FOneSkyShowImportTaskOperation>(Command->Operation);

		int32 InProjectId = -1;
		int32 InImportId = -1;
		int32 InExecutionDelayInSeconds = -1;

		if (ShowImportTaskOp.IsValid())
		{
			InProjectId = ShowImportTaskOp->GetInProjectId();
			InImportId = ShowImportTaskOp->GetInImportId();
			InExecutionDelayInSeconds = ShowImportTaskOp->GetInExecutionDelayInSeconds();

			if (OutShowImportTaskResponse.data.status == "in-progress")
			{
				FShowImportTaskQueueItem ImportTaskQueueItem;
				ImportTaskQueueItem.ImportId = InImportId;
				ImportTaskQueueItem.ProjectId = InProjectId;
				ImportTaskQueueItem.ExecutionDelayInSeconds = InExecutionDelayInSeconds;
				ImportTaskQueueItem.CreatedTimestamp = FDateTime::UtcNow();
				FOneSkyLocalizationServiceModule::Get().GetProvider().EnqueShowImportTask(ImportTaskQueueItem);
			}
			else if (OutShowImportTaskResponse.data.status == "failed")
			{
				FString CultureName = OutShowImportTaskResponse.data.file.locale.code;
				FString TargetName = OutShowImportTaskResponse.data.file.name;
				FText FailureText = FText::Format(LOCTEXT("ImportTaskFailed", "{0} translations for {1} target failed to import to OneSky!"), FText::FromString(CultureName), FText::FromString(TargetName));
				FMessageLog LocalizationServiceMessageLog("TranslationEditor");
				LocalizationServiceMessageLog.Error(FailureText);
				LocalizationServiceMessageLog.Notify(FailureText, EMessageSeverity::Error, true);
			}
		}
	}

	if (Command != nullptr)
	{
		Command->bCommandSuccessful = bResult;
		FPlatformAtomics::InterlockedExchange(&(Command->bExecuteProcessed), 1);
	}
}

// end SHOW IMPORT TASK


//FName FOneSkyListProjectTypesWorker::GetName() const
//{
//	return FName(TEXT("FOneSkyListProjectTypesWorker"));
//}
//
//bool FOneSkyListProjectTypesWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
//{
//	return false;
//}
//
//FName FOneSkyGetTranslationFileWorker::GetName() const
//{
//	return FName(TEXT("FOneSkyGetTranslationFileWorker"));
//}
//
//bool FOneSkyGetTranslationFileWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
//{
//	return false;
//}
//
//FName FOneSkyShowPhraseCollectionWorker::GetName() const
//{
//	return FName(TEXT("FOneSkyShowPhraseCollectionWorker"));
//}
//
//bool FOneSkyShowPhraseCollectionWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
//{
//	UpdateCachedLocalizationStates(OutStates);
//	return false;
//}
//
//FName FOneSkyImportPhraseCollectionWorker::GetName() const
//{
//	return FName(TEXT("FOneSkyShowPhraseCollectionWorker"));
//}
//
//bool FOneSkyImportPhraseCollectionWorker::Execute(class FOneSkyLocalizationServiceCommand& InCommand)
//{
//	return false;
//}


#undef LOCTEXT_NAMESPACE


