// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILocalizationServiceOperation.h"
#include "IOneSkyLocalizationServiceWorker.h"
#include "Misc/DateTime.h"
#include "OneSkyLocalizationServiceResponseTypes.h"

#define LOCTEXT_NAMESPACE "LocalizationService"

class FOneSkyLocalizationServiceCommand;
class IHttpRequest;
class IHttpResponse;

typedef TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> FHttpRequestPtr;
typedef TSharedPtr<class IHttpResponse, ESPMode::ThreadSafe> FHttpResponsePtr;

class FOneSkyConnect : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyConnect() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
public:

};


// LIST PROJECT GROUPS

/**
* Operation used to list project groups in one sky
*/
class FOneSkyListProjectGroupsOperation : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "ListProjectGroups";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("OneSkyLocalizationService_ListProjectGroupsWorker", "Listing Project Groups in OneSky...");
	}

	FOneSkyListProjectGroupsOperation() : InStartPage(-1), InItemsPerPage(-1) { }

	void SetInStartPage(int32 NewStartPage)
	{
		InStartPage = NewStartPage;
	}

	int32 GetInStartPage()
	{
		return InStartPage;
	}

	void SetInItemsPerPage(int32 NewItemsPerPage)
	{
		InItemsPerPage = NewItemsPerPage;
	}

	int32 GetInItemsPerPage()
	{
		return InItemsPerPage;
	}

protected:
	
	int32 InStartPage;
	int32 InItemsPerPage;
};

class FOneSkyListProjectGroupsWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyListProjectGroupsWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the  command */
	FOneSkyListProjectGroupsResponse OutListProjectGroupsResponse;
};

// end LIST PROJECT GROUPS



// SHOW PROJECT GROUP

/**
* Operation used to show a project group in one sky
*/
class FOneSkyShowProjectGroupOperation : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "ShowProjectGroup";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("OneSkyLocalizationService_ShowProjectGroupWorker", "Showing Project Group in OneSky...");
	}

	FOneSkyShowProjectGroupOperation() : InProjectGroupId(-1) { }

	int32 GetInProjectGroupId()
	{
		return InProjectGroupId;
	}

	void SetInProjectGroupId(int32 NewProjectGroupId)
	{
		InProjectGroupId = NewProjectGroupId;
	}

protected:

	int32 InProjectGroupId;
};

class FOneSkyShowProjectGroupWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyShowProjectGroupWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the command */
	FOneSkyShowProjectGroupResponse OutShowProjectGroupResponse;
};

// end SHOW PROJECT GROUP


// CREATE PROJECT GROUP

/**
* Operation used to create a project group in one sky
*/
class FOneSkyCreateProjectGroupOperation : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "CreateProjectGroup";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("OneSkyLocalizationService_CreateProjectGroupWorker", "Creating Project Group in OneSky...");
	}

	FOneSkyCreateProjectGroupOperation() : InProjectGroupName(""), InProjectGroupBaseLocale("") {}

	FString GetInProjectGroupName()
	{
		return InProjectGroupName;
	}

	void SetInProjectGroupName(FString NewProjectGroupName)
	{
		InProjectGroupName = NewProjectGroupName;
	}

	FString GetInProjectGroupBaseLocale()
	{
		return InProjectGroupBaseLocale;
	}

	void SetInInProjectGroupBaseLocale(FString NewProjectGroupBaseLocale)
	{
		InProjectGroupBaseLocale = NewProjectGroupBaseLocale;
	}

protected:

	FString InProjectGroupName;
	FString InProjectGroupBaseLocale;

};

class FOneSkyCreateProjectGroupWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyCreateProjectGroupWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the command */
	FOneSkyCreateProjectGroupResponse OutCreateProjectGroupResponse;
};

// end CREATE PROJECT GROUP


// LIST PROJECT GROUP LANGUAGES

/**
* Operation used to list the languages supported by a project group in one sky
*/
class FOneSkyListProjectGroupLanguagesOperation : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "ListProjectGroupLanguages";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("OneSkyLocalizationService_ListProjectGroupLanguagesWorker", "Listing languages for Project Group in OneSky...");
	}

	FOneSkyListProjectGroupLanguagesOperation() : InProjectGroupId(-1) { }

	int32 GetInProjectGroupId()
	{
		return InProjectGroupId;
	}

	void SetInProjectGroupId(int32 NewProjectGroupId)
	{
		InProjectGroupId = NewProjectGroupId;
	}

protected:

	int32 InProjectGroupId;
};

class FOneSkyListProjectGroupLanguagesWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyListProjectGroupLanguagesWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the command */
	FOneSkyListProjectGroupLanguagesResponse OutListProjectGroupLanguagesResponse;
};

// end LIST PROJECT GROUP LANGUAGES


// LIST PROJECTS IN GROUP

/**
* Operation used to list projects in a project group in one sky
*/
class FOneSkyListProjectsInGroupOperation : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "ListProjectsInGroup";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("OneSkyLocalizationService_ListProjectsInGroupWorker", "Listing Projects in Groups in OneSky...");
	}

	FOneSkyListProjectsInGroupOperation() : InProjectGroupId(-1) { }

	int32 GetInProjectGroupId()
	{
		return InProjectGroupId;
	}

	void SetInProjectGroupId(int32 NewProjectGroupId)
	{
		InProjectGroupId = NewProjectGroupId;
	}

protected:

	int32 InProjectGroupId;
};

class FOneSkyListProjectsInGroupWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyListProjectsInGroupWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the get project groups command */
	FOneSkyListProjectsInGroupResponse OutListProjectsInGroupResponse;
};

// end LIST PROJECTS IN GROUP

// SHOW PROJECT

/**
* Operation used to show a project in one sky
*/
class FOneSkyShowProjectOperation : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "ShowProject";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("OneSkyLocalizationService_ShowProjectWorker", "Showing Project in OneSky...");
	}

	FOneSkyShowProjectOperation() : InProjectId(-1) { }

	int32 GetInProjectId()
	{
		return InProjectId;
	}

	void SetInProjectId(int32 NewProjectId)
	{
		InProjectId = NewProjectId;
	}

protected:

	int32 InProjectId;
};

class FOneSkyShowProjectWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyShowProjectWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the command */
	FOneSkyShowProjectResponse OutShowProjectResponse;
};

// end SHOW PROJECT



// CREATE PROJECT

/**
* Operation used to create a project in one sky
*/
class FOneSkyCreateProjectOperation : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "CreateProject";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("OneSkyLocalizationService_CreateProjectWorker", "Creating Project in OneSky...");
	}

	FOneSkyCreateProjectOperation() : InProjectGroupId(-1), InProjectType(""), InProjectName(""), InProjectDescription("") {}

	FString GetInProjectGroupName()
	{
		return InProjectName;
	}

	void SetInProjectGroupName(FString NewProjectName)
	{
		InProjectName = NewProjectName;
	}

	FString GetInProjectDescription()
	{
		return InProjectDescription;
	}

	void SetInInProjectDescription(FString NewProjectDescription)
	{
		InProjectDescription = NewProjectDescription;
	}

	FString GetInProjectType()
	{
		return InProjectType;
	}

	void SetInInProjectType(FString NewProjectType)
	{
		InProjectType = NewProjectType;
	}

	int32 GetInProjectGroupId()
	{
		return InProjectGroupId;
	}

	void SetInProjectGroupId(int32 NewProjectGroupId)
	{
		InProjectGroupId = NewProjectGroupId;
	}

protected:

	int32 InProjectGroupId;	// Project Group this project will belong to
	FString InProjectType; // See https://github.com/onesky/api-documentation-platform/blob/master/resources/project_type.md
	FString InProjectName;
	FString InProjectDescription;

};

class FOneSkyCreateProjectWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyCreateProjectWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the command */
	FOneSkyCreateProjectResponse OutCreateProjectResponse;
};

// end CREATE PROJECT



// LIST PROJECT GROUP LANGUAGES

/**
* Operation used to list the languages supported by a project in one sky
*/
class FOneSkyListProjectLanguagesOperation : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "ListProjectLanguages";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("OneSkyLocalizationService_ListProjectLanguagesWorker", "Listing languages for Project in OneSky...");
	}

	FOneSkyListProjectLanguagesOperation() : InProjectId(-1) { }

	int32 GetInProjectId()
	{
		return InProjectId;
	}

	void SetInProjectId(int32 NewProjectGroupId)
	{
		InProjectId = NewProjectGroupId;
	}

protected:

	int32 InProjectId;
};

class FOneSkyListProjectLanguagesWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyListProjectLanguagesWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the command */
	FOneSkyListProjectLanguagesResponse OutListProjectLanguagesResponse;
};

// end LIST PROJECT GROUP LANGUAGES


// TRANSLATION STATUS

/**
* Operation used to list the languages supported by a project in one sky
*/
class FOneSkyTranslationStatusOperation : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "TranslationStatus";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("OneSkyLocalizationService_TranslationStatus", "Retrieving the translation status for a file in a project in OneSky...");
	}

	FOneSkyTranslationStatusOperation() : InProjectId(-1) { }

	int32 GetInProjectId()
	{
		return InProjectId;
	}

	void SetInProjectId(int32 NewProjectGroupId)
	{
		InProjectId = NewProjectGroupId;
	}

	FString GetInFileName()
	{
		return InFileName;
	}

	void SetInFileName(FString NewFileName)
	{
		InFileName = NewFileName;
	}

	FString GetInLocale()
	{
		return InLocale;
	}

	void SetInLocale(FString NewLocale)
	{
		InLocale = NewLocale;
	}

	FString GetOutPercentComplete()
	{
		return OutPercentComplete;
	}

	void SetOutPercentComplete(FString NewOutPercentComplete)
	{
		OutPercentComplete = NewOutPercentComplete;
	}

protected:

	int32 InProjectId;
	FString InFileName;
	FString InLocale;
	FString OutPercentComplete;
};

class FOneSkyTranslationStatusWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyTranslationStatusWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the command */
	FOneSkyTranslationStatusResponse OutTranslationStatusResponse;
};

// end TRANSLATION STATUS


// TRANSLATION EXPORT

class FOneSkyTranslationExportWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyTranslationExportWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the command */
	//FOneSkyTranslationExportResponse OutTranslationExportResponse;
};

// end TRANSLATION EXPORT


// LIST UPLOADED FILES

/**
* Operation used to list uploaded files for a project in one sky
*/
class FOneSkyListUploadedFilesOperation : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "ListUploadedFiles";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("OneSkyLocalizationService_ListUploadedFilesWorker", "Listing Uploaded Files for a Project in OneSky...");
	}

	FOneSkyListUploadedFilesOperation() : InProjectId(-1), InStartPage(-1), InItemsPerPage(-1) { }

	void SetInStartPage(int32 NewStartPage)
	{
		InStartPage = NewStartPage;
	}

	int32 GetInStartPage()
	{
		return InStartPage;
	}

	void SetInItemsPerPage(int32 NewItemsPerPage)
	{
		InItemsPerPage = NewItemsPerPage;
	}

	int32 GetInItemsPerPage()
	{
		return InItemsPerPage;
	}

	int32 GetInProjectId()
	{
		return InProjectId;
	}

	void SetInProjectId(int32 NewProjectGroupId)
	{
		InProjectId = NewProjectGroupId;
	}

protected:

	int32 InProjectId;
	int32 InStartPage;
	int32 InItemsPerPage;
};

class FOneSkyListUploadedFilesWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyListUploadedFilesWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the  command */
	FOneSkyListUploadedFilesResponse OutListUploadedFilesResponse;
};

// end LIST UPLOADED FILES


//// UPLOAD FILE

class FOneSkyUploadFileWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyUploadFileWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the  command */
	FOneSkyUploadFileResponse OutUploadFileResponse;
};

// end UPLOAD FILE


// LIST PHRASE COLLECTIONS

/**
* Operation used to list phrase collections for a project in one sky
*/
class FOneSkyListPhraseCollectionsOperation : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "ListPhraseCollections";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("OneSkyLocalizationService_ListPhraseCollections", "Listing Phrase Collections for a Project in OneSky...");
	}

	FOneSkyListPhraseCollectionsOperation() : InProjectId(-1), InPage(-1), InItemsPerPage(-1) { }

	void SetInPage(int32 NewPage)
	{
		InPage = NewPage;
	}

	int32 GetInPage()
	{
		return InPage;
	}

	void SetInItemsPerPage(int32 NewItemsPerPage)
	{
		InItemsPerPage = NewItemsPerPage;
	}

	int32 GetInItemsPerPage()
	{
		return InItemsPerPage;
	}

	int32 GetInProjectId()
	{
		return InProjectId;
	}

	void SetInProjectId(int32 NewProjectGroupId)
	{
		InProjectId = NewProjectGroupId;
	}

protected:

	int32 InProjectId;
	int32 InPage;
	int32 InItemsPerPage;
};

class FOneSkyListPhraseCollectionsWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyListPhraseCollectionsWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the  command */
	FOneSkyListPhraseCollectionsResponse OutListPhraseCollectionsResponse;
};

// end LIST PHRASE COLLECTIONS


// SHOW IMPORT TASK

/**
* Operation used to show a project in one sky
*/
class FOneSkyShowImportTaskOperation : public ILocalizationServiceOperation
{
public:
	// ILocalizationServiceOperation interface
	virtual FName GetName() const override
	{
		return "ShowImportTask";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("OneSkyLocalizationService_ShowImportTask", "Showing Import Task in OneSky...");
	}

	FOneSkyShowImportTaskOperation() : InProjectId(-1) { }

	int32 GetInProjectId()
	{
		return InProjectId;
	}

	void SetInProjectId(int32 NewProjectId)
	{
		InProjectId = NewProjectId;
	}

	int32 GetInImportId()
	{
		return InImportId;
	}

	void SetInImportId(int32 NewImportId)
	{
		InImportId = NewImportId;
	}

	int32 GetInExecutionDelayInSeconds()
	{
		return InExecutionDelayInSeconds;
	}

	void SetInExecutionDelayInSeconds(int32 NewInExecutionDelayInSeconds)
	{
		InExecutionDelayInSeconds = NewInExecutionDelayInSeconds;
	}

	FDateTime GetInCreationTimestamp()
	{
		return InCreationTimestamp;
	}

	void SetInCreationTimestamp(FDateTime NewInCreationTimestamp)
	{
		InCreationTimestamp = NewInCreationTimestamp;
	}

protected:

	int32 InProjectId;
	int32 InImportId;
	int32 InExecutionDelayInSeconds;
	FDateTime InCreationTimestamp;
};

class FOneSkyShowImportTaskWorker : public IOneSkyLocalizationServiceWorker
{
public:
	virtual ~FOneSkyShowImportTaskWorker() {}
	// IOneSkyLocalizationServiceWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
	virtual bool UpdateStates() const override {/*TODO:*/return true; }
	void Query_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
public:

	FOneSkyLocalizationServiceCommand* Command;

	/** Response to the command */
	FOneSkyShowImportTaskResponse OutShowImportTaskResponse;
};

// end SHOW IMPORT TASK

//class FOneSkyListProjectTypesWorker : public IOneSkyLocalizationServiceWorker
//{
//public:
//	virtual ~FOneSkyListProjectTypesWorker() {}
//	// IOneSkyLocalizationServiceWorker interface
//	virtual FName GetName() const override;
//	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
//	virtual bool UpdateStates() const override {/*TODO:*/return true; }
//public:
//	/** Map of filenames to perforce state */
//	//FOneSkeyProjectTypeResponse OutResults;
//};
//
//class FOneSkyShowPhraseCollectionWorker : public IOneSkyLocalizationServiceWorker
//{
//public:
//	virtual ~FOneSkyShowPhraseCollectionWorker() 
//	{}
//	// IOneSkyLocalizationServiceWorker interface
//	virtual FName GetName() const override;
//	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
//	virtual bool UpdateStates() const override {/*TODO:*/return true; }
//public:
//	int32 InCollectionKey;
//
//	TMap<FLocalizationServiceTranslationIdentifier, TSharedRef<FOneSkyLocalizationServiceState, ESPMode::ThreadSafe>, FDefaultSetAllocator, FLocalizationServiceTranslationIdentifierKeyFuncs<TSharedRef<FOneSkyLocalizationServiceState, ESPMode::ThreadSafe>>> OutStates;
//	//FOneSkyShowPhraseCollectionResponse ShowPhraseCollectionResponse;
//};
//
//class FOneSkyImportPhraseCollectionWorker : public IOneSkyLocalizationServiceWorker
//{
//public:
//	virtual ~FOneSkyImportPhraseCollectionWorker() {}
//	// IOneSkyLocalizationServiceWorker interface
//	virtual FName GetName() const override;
//	virtual bool Execute(class FOneSkyLocalizationServiceCommand& InCommand) override;
//	virtual bool UpdateStates() const override {/*TODO:*/return true; }
//public:
//	//FOneSkyPhraseCollection InCollection;
//
//	// Whether to deprecate strings within an previously created phrase collection that cannot be found in the re - creation of the phrase collection.
//	bool InKeepAllStrings;	
//
//	/** Map of filenames to perforce state */
//	//FOneSkyImportPhraseCollectionResponse
//};

#undef LOCTEXT_NAMESPACE
