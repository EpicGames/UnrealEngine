// Copyright Epic Games, Inc. All Rights Reserved.

#include "OneSkyLocalizationServiceProvider.h"
#include "Framework/Commands/UICommandList.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Misc/FeedbackContext.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Framework/Commands/Commands.h"
#include "OneSkyLocalizationServiceState.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "ILocalizationServiceModule.h"
#include "OneSkyLocalizationServiceModule.h"
#include "OneSkyLocalizationServiceCommand.h"
#include "OneSkyLocalizationServiceOperations.h"
#include "OneSkyConnection.h"
#include "Logging/MessageLog.h"
#include "Widgets/SWindow.h"
#if LOCALIZATION_SERVICES_WITH_SLATE

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/QueuedThreadPool.h"
#include "DetailWidgetRow.h"

#endif
#include "LocalizationTargetTypes.h"
#include "LocalizationCommandletTasks.h"
#include "LocalizationModule.h"
#include "Interfaces/IMainFrameModule.h"

static FName ProviderName("OneSky");

#define LOCTEXT_NAMESPACE "OneSkyLocalizationService"

class FOneSkyLocalizationTargetEditorCommands : public TCommands<FOneSkyLocalizationTargetEditorCommands>
{
public:
	FOneSkyLocalizationTargetEditorCommands()
		: TCommands<FOneSkyLocalizationTargetEditorCommands>("OneSkyLocalizationTargetEditor", NSLOCTEXT("OneSky", "OneSkyLocalizationTargetEditor", "OneSky Localization Target Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
		}

	TSharedPtr<FUICommandInfo> ImportAllCulturesForTargetFromOneSky;
	TSharedPtr<FUICommandInfo> ExportAllCulturesForTargetToOneSky;
	TSharedPtr<FUICommandInfo> ImportAllTargetsAllCulturesForTargetSetFromOneSky;
	TSharedPtr<FUICommandInfo> ExportAllTargetsAllCulturesForTargetSetFromOneSky;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};

void FOneSkyLocalizationTargetEditorCommands::RegisterCommands()
{
	UI_COMMAND(ImportAllCulturesForTargetFromOneSky, "Import All Cultures from OneSky", "Imports translations for all cultures of this target to OneSky.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportAllCulturesForTargetToOneSky, "Export All Cultures to OneSky", "Exports translations for all cultures of this target to OneSky.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ImportAllTargetsAllCulturesForTargetSetFromOneSky, "Import All Targets from OneSky", "Imports translations for all cultures of all targets of this target set to OneSky.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportAllTargetsAllCulturesForTargetSetFromOneSky, "Export All Targets to OneSky", "Exports translations for all cultures of all targets of this target set to OneSky.", EUserInterfaceActionType::Button, FInputChord());
}

/** Init of connection with source control server */
void FOneSkyLocalizationServiceProvider::Init(bool bForceConnection)
{
	// TODO: Test if connection works?
	bServerAvailable = true;

	FOneSkyLocalizationTargetEditorCommands::Register();
}

/** API Specific close the connection with localization provider server*/
void FOneSkyLocalizationServiceProvider::Close()
{
	if ( PersistentConnection )
	{
		PersistentConnection->Disconnect();
		delete PersistentConnection;
		PersistentConnection = NULL;
	}

	// clear the cache
	StateCache.Empty();

	bServerAvailable = false;
}

TSharedRef<FOneSkyLocalizationServiceState, ESPMode::ThreadSafe> FOneSkyLocalizationServiceProvider::GetStateInternal(const FLocalizationServiceTranslationIdentifier& InTranslationId)
{
	TSharedRef<FOneSkyLocalizationServiceState, ESPMode::ThreadSafe>* State = StateCache.Find(InTranslationId);
	if(State != NULL)
	{
		// found cached item
		return (*State);
	}
	else
	{
		// cache an unknown state for this item
		TSharedRef<FOneSkyLocalizationServiceState, ESPMode::ThreadSafe> NewState = MakeShareable(new FOneSkyLocalizationServiceState(InTranslationId));
		StateCache.Add(InTranslationId, NewState);
		return NewState;
	}
}

FText FOneSkyLocalizationServiceProvider::GetStatusText() const
{
	FOneSkyLocalizationServiceModule& OneSkyLocalizationService = FModuleManager::LoadModuleChecked<FOneSkyLocalizationServiceModule>( "OneSkyLocalizationService" );
	const FOneSkyLocalizationServiceSettings& Settings = OneSkyLocalizationService.AccessSettings();

	FFormatNamedArguments Args;
	Args.Add( TEXT("IsEnabled"), IsEnabled() ? LOCTEXT("Yes", "Yes") : LOCTEXT("No", "No") );
	Args.Add( TEXT("IsConnected"), (IsEnabled() && IsAvailable()) ? LOCTEXT("Yes", "Yes") : LOCTEXT("No", "No") );
	Args.Add( TEXT("ConnectionName"), FText::FromString( Settings.GetConnectionName() ) );

	return FText::Format( LOCTEXT("OneSkyStatusText", "Enabled: {IsEnabled}\nConnected: {IsConnected}\nConnectionName: {ConnectionName}\n"), Args );
}

bool FOneSkyLocalizationServiceProvider::IsEnabled() const
{
	return true;
}

bool FOneSkyLocalizationServiceProvider::IsAvailable() const
{
	return bServerAvailable;
}

bool FOneSkyLocalizationServiceProvider::EstablishPersistentConnection()
{
	FOneSkyLocalizationServiceModule& OneSkyLocalizationService = FModuleManager::LoadModuleChecked<FOneSkyLocalizationServiceModule>( "OneSkyLocalizationService" );
	FOneSkyConnectionInfo ConnectionInfo = OneSkyLocalizationService.AccessSettings().GetConnectionInfo();

	bool bIsValidConnection = false;
	if ( !PersistentConnection )
	{
		PersistentConnection = new FOneSkyConnection(ConnectionInfo);
	}

	bIsValidConnection = PersistentConnection->IsValidConnection();
	if ( !bIsValidConnection )
	{
		delete PersistentConnection;
		PersistentConnection = new FOneSkyConnection(ConnectionInfo);
		bIsValidConnection = PersistentConnection->IsValidConnection();
	}

	bServerAvailable = bIsValidConnection;
	return bIsValidConnection;
}

const FName& FOneSkyLocalizationServiceProvider::GetName() const
{
	return ProviderName;
}

const FText FOneSkyLocalizationServiceProvider::GetDisplayName() const
{
	return LOCTEXT("OneSkyLocalizationService", "OneSky Localization Service");
}

ELocalizationServiceOperationCommandResult::Type FOneSkyLocalizationServiceProvider::GetState(const TArray<FLocalizationServiceTranslationIdentifier>& InTranslationIds, TArray< TSharedRef<ILocalizationServiceState, ESPMode::ThreadSafe> >& OutState, ELocalizationServiceCacheUsage::Type InStateCacheUsage)
{
	if(!IsEnabled())
	{
		return ELocalizationServiceOperationCommandResult::Failed;
	}

	if (InStateCacheUsage == ELocalizationServiceCacheUsage::ForceUpdate)
	{
		// TODO: Something like this?
		//for (TArray<FText>::TConstIterator It(InTexts); It; It++)
		{
			//TSharedRef<FOneSkyShowPhraseCollectionWorker, ESPMode::ThreadSafe> PhraseCollectionWorker = ILocalizationServiceOperation::Create<FOneSkyShowPhraseCollectionWorker>();

			//PhraseCollectionWorker->InCollectionKey = FTextInspector::GetKey(*It);
			//Execute(ILocalizationServiceOperation::Create<FLocalizationServiceUpdateStatus>(), InTranslationIds);
		}
	}

	for (TArray<FLocalizationServiceTranslationIdentifier>::TConstIterator It(InTranslationIds); It; It++)
	{
		TSharedRef<FOneSkyLocalizationServiceState, ESPMode::ThreadSafe>* State = StateCache.Find(*It);
		if(State != NULL)
		{
			// found cached item for this file, return that
			OutState.Add(*State);
		}
		else
		{
			// cache an unknown state for this item & return that
			TSharedRef<FOneSkyLocalizationServiceState, ESPMode::ThreadSafe> NewState = MakeShareable(new FOneSkyLocalizationServiceState(*It));
			StateCache.Add(*It, NewState);
			OutState.Add(NewState);
		}
	}

	return ELocalizationServiceOperationCommandResult::Succeeded;
}

ELocalizationServiceOperationCommandResult::Type FOneSkyLocalizationServiceProvider::Execute(const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation, const TArray<FLocalizationServiceTranslationIdentifier>& InTranslationIds, ELocalizationServiceOperationConcurrency::Type InConcurrency, const FLocalizationServiceOperationComplete& InOperationCompleteDelegate)
{
	if(!IsEnabled())
	{
		return ELocalizationServiceOperationCommandResult::Failed;
	}

	// Query to see if the we allow this operation
	TSharedPtr<IOneSkyLocalizationServiceWorker, ESPMode::ThreadSafe> Worker = CreateWorker(InOperation->GetName());
	if(!Worker.IsValid())
	{
		// this operation is unsupported by this source control provider
		FFormatNamedArguments Arguments;
		Arguments.Add( TEXT("OperationName"), FText::FromName(InOperation->GetName()) );
		Arguments.Add( TEXT("ProviderName"), FText::FromName(GetName()) );
		FMessageLog("LocalizationService").Error(FText::Format(LOCTEXT("UnsupportedOperation", "Operation '{OperationName}' not supported by revision control provider '{ProviderName}'"), Arguments));
		return ELocalizationServiceOperationCommandResult::Failed;
	}

	// fire off operation
	if(InConcurrency == ELocalizationServiceOperationConcurrency::Synchronous)
	{
		FOneSkyLocalizationServiceCommand* Command = new FOneSkyLocalizationServiceCommand(InOperation, Worker.ToSharedRef());
		Command->bAutoDelete = false;
		Command->OperationCompleteDelegate = InOperationCompleteDelegate;
		return ExecuteSynchronousCommand(*Command, InOperation->GetInProgressString(), true);
	}
	else
	{
		FOneSkyLocalizationServiceCommand* Command = new FOneSkyLocalizationServiceCommand(InOperation, Worker.ToSharedRef());
		Command->bAutoDelete = true;
		Command->OperationCompleteDelegate = InOperationCompleteDelegate;
		return IssueCommand(*Command, false);
	}
}

bool FOneSkyLocalizationServiceProvider::CanCancelOperation( const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation ) const
{
	for (int32 CommandIndex = 0; CommandIndex < CommandQueue.Num(); ++CommandIndex)
	{
		FOneSkyLocalizationServiceCommand& Command = *CommandQueue[CommandIndex];
		if (Command.Operation == InOperation)
		{
			check(Command.bAutoDelete);
			return true;
		}
	}

	// operation was not in progress!
	return false;
}

void FOneSkyLocalizationServiceProvider::CancelOperation( const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation )
{
	for (int32 CommandIndex = 0; CommandIndex < CommandQueue.Num(); ++CommandIndex)
	{
		FOneSkyLocalizationServiceCommand& Command = *CommandQueue[CommandIndex];
		if (Command.Operation == InOperation)
		{
			check(Command.bAutoDelete);
			Command.Cancel();
			return;
		}
	}
}

void FOneSkyLocalizationServiceProvider::OutputCommandMessages(const FOneSkyLocalizationServiceCommand& InCommand) const
{
	FMessageLog LocalizationServiceLog("LocalizationService");

	for (int32 ErrorIndex = 0; ErrorIndex < InCommand.ErrorMessages.Num(); ++ErrorIndex)
	{
		LocalizationServiceLog.Error(InCommand.ErrorMessages[ErrorIndex]);
	}

	for (int32 InfoIndex = 0; InfoIndex < InCommand.InfoMessages.Num(); ++InfoIndex)
	{
		LocalizationServiceLog.Info(InCommand.InfoMessages[InfoIndex]);
	}
}

void FOneSkyLocalizationServiceProvider::Tick()
{	
	bool bStatesUpdated = false;

	while (!ShowImportTaskQueue.IsEmpty())
	{
		FShowImportTaskQueueItem ImportTaskItem;
		ShowImportTaskQueue.Dequeue(ImportTaskItem);
		TSharedRef<FOneSkyShowImportTaskOperation, ESPMode::ThreadSafe> ShowImportTaskOperation = ILocalizationServiceOperation::Create<FOneSkyShowImportTaskOperation>();
		ShowImportTaskOperation->SetInProjectId(ImportTaskItem.ProjectId);
		ShowImportTaskOperation->SetInImportId(ImportTaskItem.ImportId);
		ShowImportTaskOperation->SetInExecutionDelayInSeconds(ImportTaskItem.ExecutionDelayInSeconds);
		ShowImportTaskOperation->SetInCreationTimestamp(ImportTaskItem.CreatedTimestamp);
		FOneSkyLocalizationServiceModule::Get().GetProvider().Execute(ShowImportTaskOperation, TArray<FLocalizationServiceTranslationIdentifier>(), ELocalizationServiceOperationConcurrency::Asynchronous);
	}

	for (int32 CommandIndex = 0; CommandIndex < CommandQueue.Num(); ++CommandIndex)
	{
		FOneSkyLocalizationServiceCommand& Command = *CommandQueue[CommandIndex];
		if (Command.bExecuteProcessed)
		{
			// Remove command from the queue
			CommandQueue.RemoveAt(CommandIndex);

			// update connection state
			bServerAvailable = !Command.bConnectionDropped || Command.bCancelled;

			// let command update the states of any files
			bStatesUpdated |= Command.Worker->UpdateStates();

			// dump any messages to output log
			OutputCommandMessages(Command);

			// run the completion delegate if we have one bound
			ELocalizationServiceOperationCommandResult::Type Result = ELocalizationServiceOperationCommandResult::Failed;
			if(Command.bCommandSuccessful)
			{
				Result = ELocalizationServiceOperationCommandResult::Succeeded;
			}
			else if(Command.bCancelled)
			{
				Result = ELocalizationServiceOperationCommandResult::Cancelled;
			}
			Command.OperationCompleteDelegate.ExecuteIfBound(Command.Operation, Result);

			//commands that are left in the array during a tick need to be deleted
			if(Command.bAutoDelete)
			{
				// Only delete commands that are not running 'synchronously'
				delete &Command;
			}

			// only do one command per tick loop, as we dont want concurrent modification 
			// of the command queue (which can happen in the completion delegate)
			break;
		}
	}

	//if(bStatesUpdated)
	//{
	//	OnLocalizationServiceStateChanged.Broadcast();
	//}
}

static void PublicKeyChanged(const FText& NewText, ETextCommit::Type CommitType)
{
	FOneSkyLocalizationServiceModule::Get().AccessSettings().SetApiKey(NewText.ToString());
	FOneSkyLocalizationServiceModule::Get().AccessSettings().SaveSettings();
}

static void SecretKeyChanged(const FText& NewText, ETextCommit::Type CommitType)
{
	FOneSkyLocalizationServiceModule::Get().AccessSettings().SetApiSecret(NewText.ToString());
	FOneSkyLocalizationServiceModule::Get().AccessSettings().SaveSettings();
}

static void SaveSecretKeyChanged(ECheckBoxState CheckBoxState)
{
	FOneSkyLocalizationServiceModule::Get().AccessSettings().SetSaveSecretKey(CheckBoxState == ECheckBoxState::Checked);
	FOneSkyLocalizationServiceModule::Get().AccessSettings().SaveSettings();
}

#if LOCALIZATION_SERVICES_WITH_SLATE
void FOneSkyLocalizationServiceProvider::CustomizeSettingsDetails(IDetailCategoryBuilder& DetailCategoryBuilder) const
{
	FOneSkyConnectionInfo ConnectionInfo = FOneSkyLocalizationServiceModule::Get().AccessSettings().GetConnectionInfo();
	FText PublicKeyText = LOCTEXT("OneSkyPublicKeyLabel", "OneSky API Public Key");
	FDetailWidgetRow& PublicKeyRow = DetailCategoryBuilder.AddCustomRow(PublicKeyText);
	PublicKeyRow.NameContent()
		[
			SNew(STextBlock)
			.Text(PublicKeyText)
		];
	PublicKeyRow.ValueContent()
		[
			SNew(SEditableTextBox)
			.OnTextCommitted(FOnTextCommitted::CreateStatic(&PublicKeyChanged))
			.Text(FText::FromString(ConnectionInfo.ApiKey))
		];

	FText SecretKeyText = LOCTEXT("OneSkySecretKeyLabel", "OneSky API Secret Key");
	FDetailWidgetRow& SecretKeyRow = DetailCategoryBuilder.AddCustomRow(SecretKeyText);
	SecretKeyRow.NameContent()
		[
			SNew(STextBlock)
			.Text(SecretKeyText)
		];
	SecretKeyRow.ValueContent()
		[
			SNew(SEditableTextBox)
			.OnTextCommitted(FOnTextCommitted::CreateStatic(&SecretKeyChanged))
			.Text(FText::FromString(ConnectionInfo.ApiSecret))
		];

	FText SaveSecretKeyText = LOCTEXT("OneSkySaveSecret", "Remember Secret Key (WARNING: saved unencrypted)");
	FDetailWidgetRow& SaveSecretKeyRow = DetailCategoryBuilder.AddCustomRow(SaveSecretKeyText);
	SaveSecretKeyRow.NameContent()
		[
			SNew(STextBlock)
			.Text(SaveSecretKeyText)
		];
	SaveSecretKeyRow.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(FOneSkyLocalizationServiceModule::Get().AccessSettings().GetSaveSecretKey() ?  ECheckBoxState::Checked : ECheckBoxState::Unchecked)
			.OnCheckStateChanged(FOnCheckStateChanged::CreateStatic(&SaveSecretKeyChanged))
		];
}

static void ProjectChanged(const FText& NewText, ETextCommit::Type CommitType, FGuid TargetGuid)
{
	FOneSkyLocalizationTargetSetting* Settings = FOneSkyLocalizationServiceModule::Get().AccessSettings().GetSettingsForTarget(TargetGuid, true);
	int32 NewProjectId = INDEX_NONE;	// Default to -1
	FString StringId = NewText.ToString();
	// Don't allow this to be set to a non-numeric value.
	if (StringId.IsNumeric())
	{
		NewProjectId = FCString::Atoi(*StringId);
	}
	FOneSkyLocalizationServiceModule::Get().AccessSettings().SetSettingsForTarget(TargetGuid, NewProjectId, Settings->OneSkyFileName);
}

static void FileNameChanged(const FText& NewText, ETextCommit::Type CommitType, FGuid TargetGuid)
{
	FOneSkyLocalizationTargetSetting* Settings = FOneSkyLocalizationServiceModule::Get().AccessSettings().GetSettingsForTarget(TargetGuid, true);
	FOneSkyLocalizationServiceModule::Get().AccessSettings().SetSettingsForTarget(TargetGuid, Settings->OneSkyProjectId, NewText.ToString());
}

void FOneSkyLocalizationServiceProvider::CustomizeTargetDetails(IDetailCategoryBuilder& DetailCategoryBuilder, TWeakObjectPtr<ULocalizationTarget> LocalizationTarget) const
{
	if (!LocalizationTarget.IsValid())
	{
		return;
	}

	FOneSkyLocalizationTargetSetting* Settings = FOneSkyLocalizationServiceModule::Get().AccessSettings().GetSettingsForTarget(LocalizationTarget->Settings.Guid, true);

	FText ProjectText = LOCTEXT("OneSkyProjectIdLabel", "OneSky Project ID");
	FDetailWidgetRow& ProjectRow = DetailCategoryBuilder.AddCustomRow(ProjectText);
	ProjectRow.NameContent()
		[
			SNew(STextBlock)
			.Text(ProjectText)
		];
	ProjectRow.ValueContent()
		[
			SNew(SEditableTextBox)
			.OnTextCommitted(FOnTextCommitted::CreateStatic(&ProjectChanged, LocalizationTarget->Settings.Guid))
			.Text_Lambda([Settings]
			{
				int32 SavedProjectId = Settings->OneSkyProjectId;
				if (SavedProjectId >= 0)
				{
					return (const FText&) FText::FromString(FString::FromInt(SavedProjectId));
				}
				
				// Show empty string if value is default (-1)
				return FText::GetEmpty();
			}
			)
		];

	FText FileText = LOCTEXT("OneSkyFileNameLabel", "OneSky File Name");
	FDetailWidgetRow& FileNameRow = DetailCategoryBuilder.AddCustomRow(FileText);
	FileNameRow.NameContent()
		[
			SNew(STextBlock)
			.Text(FileText)
		];
	FileNameRow.ValueContent()
		[
			SNew(SEditableTextBox)
			.OnTextCommitted(FOnTextCommitted::CreateStatic(&FileNameChanged, LocalizationTarget->Settings.Guid))
			.Text(FText::FromString(Settings->OneSkyFileName))
		];
}

void FOneSkyLocalizationServiceProvider::CustomizeTargetToolbar(TSharedRef<FExtender>& MenuExtender, TWeakObjectPtr<ULocalizationTarget> LocalizationTarget) const
{
	const TSharedRef< FUICommandList > CommandList = MakeShareable(new FUICommandList);

	MenuExtender->AddToolBarExtension("LocalizationService", EExtensionHook::First, CommandList,
		FToolBarExtensionDelegate::CreateRaw(const_cast<FOneSkyLocalizationServiceProvider*>(this), &FOneSkyLocalizationServiceProvider::AddTargetToolbarButtons, LocalizationTarget, CommandList));

}

void FOneSkyLocalizationServiceProvider::AddTargetToolbarButtons(FToolBarBuilder& ToolbarBuilder, TWeakObjectPtr<ULocalizationTarget> InLocalizationTarget, TSharedRef< FUICommandList > CommandList)
{
	bool bIsTargetSet = false;
	CommandList->MapAction(FOneSkyLocalizationTargetEditorCommands::Get().ImportAllCulturesForTargetFromOneSky, FExecuteAction::CreateRaw(this, &FOneSkyLocalizationServiceProvider::ImportAllCulturesForTargetFromOneSky, InLocalizationTarget, bIsTargetSet));
	ToolbarBuilder.AddToolBarButton(FOneSkyLocalizationTargetEditorCommands::Get().ImportAllCulturesForTargetFromOneSky, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "TranslationEditor.ImportLatestFromLocalizationService"));

	// Don't add "export all" buttons for engine targets
	if (!InLocalizationTarget->IsMemberOfEngineTargetSet())
	{
		CommandList->MapAction(FOneSkyLocalizationTargetEditorCommands::Get().ExportAllCulturesForTargetToOneSky, FExecuteAction::CreateRaw(this, &FOneSkyLocalizationServiceProvider::ExportAllCulturesForTargetToOneSky, InLocalizationTarget, bIsTargetSet));
		ToolbarBuilder.AddToolBarButton(FOneSkyLocalizationTargetEditorCommands::Get().ExportAllCulturesForTargetToOneSky, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "TranslationEditor.ImportLatestFromLocalizationService"));
	}
}

void FOneSkyLocalizationServiceProvider::CustomizeTargetSetToolbar(TSharedRef<FExtender>& MenuExtender, TWeakObjectPtr<ULocalizationTargetSet> InLocalizationTargetSet) const
{
	const TSharedRef< FUICommandList > CommandList = MakeShareable(new FUICommandList);

	MenuExtender->AddToolBarExtension("LocalizationService", EExtensionHook::First, CommandList,
		FToolBarExtensionDelegate::CreateRaw(const_cast<FOneSkyLocalizationServiceProvider*>(this), &FOneSkyLocalizationServiceProvider::AddTargetSetToolbarButtons, InLocalizationTargetSet, CommandList));

}

void FOneSkyLocalizationServiceProvider::AddTargetSetToolbarButtons(FToolBarBuilder& ToolbarBuilder, TWeakObjectPtr<ULocalizationTargetSet> InLocalizationTargetSet, TSharedRef< FUICommandList > CommandList)
{
	CommandList->MapAction(FOneSkyLocalizationTargetEditorCommands::Get().ImportAllTargetsAllCulturesForTargetSetFromOneSky, FExecuteAction::CreateRaw(this, &FOneSkyLocalizationServiceProvider::ImportAllTargetsForTargetSetFromOneSky, InLocalizationTargetSet));
	ToolbarBuilder.AddToolBarButton(FOneSkyLocalizationTargetEditorCommands::Get().ImportAllTargetsAllCulturesForTargetSetFromOneSky, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "TranslationEditor.ImportLatestFromLocalizationService"));

	// Don't add "export all" button for the engine target set
	if (InLocalizationTargetSet.IsValid() && InLocalizationTargetSet->TargetObjects.Num() > 0 && !(InLocalizationTargetSet->TargetObjects[0]->IsMemberOfEngineTargetSet()))
	{
		CommandList->MapAction(FOneSkyLocalizationTargetEditorCommands::Get().ExportAllTargetsAllCulturesForTargetSetFromOneSky, FExecuteAction::CreateRaw(this, &FOneSkyLocalizationServiceProvider::ExportAllTargetsForTargetSetToOneSky, InLocalizationTargetSet));
		ToolbarBuilder.AddToolBarButton(FOneSkyLocalizationTargetEditorCommands::Get().ExportAllTargetsAllCulturesForTargetSetFromOneSky, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "TranslationEditor.ImportLatestFromLocalizationService"));
	}
}


#endif //LOCALIZATION_SERVICES_WITH_SLATE

TSharedPtr<IOneSkyLocalizationServiceWorker, ESPMode::ThreadSafe> FOneSkyLocalizationServiceProvider::CreateWorker(const FName& InOperationName) const
{
	const FGetOneSkyLocalizationServiceWorker* Operation = WorkersMap.Find(InOperationName);
	if(Operation != NULL)
	{
		return Operation->Execute();
	}
		
	return NULL;
}

void FOneSkyLocalizationServiceProvider::RegisterWorker( const FName& InName, const FGetOneSkyLocalizationServiceWorker& InDelegate )
{
	WorkersMap.Add( InName, InDelegate );
}

ELocalizationServiceOperationCommandResult::Type FOneSkyLocalizationServiceProvider::ExecuteSynchronousCommand(FOneSkyLocalizationServiceCommand& InCommand, const FText& Task, bool bSuppressResponseMsg)
{
	ELocalizationServiceOperationCommandResult::Type Result = ELocalizationServiceOperationCommandResult::Failed;

	struct Local
	{
		static void CancelCommand(FOneSkyLocalizationServiceCommand* InControlCommand)
		{
			InControlCommand->Cancel();
		}
	};

	// Display the progress dialog
	//FScopedLocalizationServiceProgress Progress(Task, FSimpleDelegate::CreateStatic(&Local::CancelCommand, &InCommand));

	// Perform the command asynchronously
	IssueCommand( InCommand, false );

	// Wait until the queue is empty. Only at this point is our command guaranteed to be removed from the queue
	while(CommandQueue.Num() > 0)
	{
		// Tick the command queue and update progress.
		Tick();

		//Progress.Tick();

		// Sleep for a bit so we don't busy-wait so much.
		FPlatformProcess::Sleep(0.01f);
	}

	if(InCommand.bCommandSuccessful)
	{
		Result = ELocalizationServiceOperationCommandResult::Succeeded;
	}
	else if(InCommand.bCancelled)
	{
		Result = ELocalizationServiceOperationCommandResult::Cancelled;
	}

	// If the command failed, inform the user that they need to try again
	if (!InCommand.bCancelled && Result != ELocalizationServiceOperationCommandResult::Succeeded && !bSuppressResponseMsg)
	{
		FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("OneSky_ServerUnresponsive", "OneSky server is unresponsive. Please check your connection and try again.") );
	}

	// Delete the command now
	check(!InCommand.bAutoDelete);
	delete &InCommand;

	return Result;
}

ELocalizationServiceOperationCommandResult::Type FOneSkyLocalizationServiceProvider::IssueCommand(FOneSkyLocalizationServiceCommand& InCommand, const bool bSynchronous)
{
	if ( !bSynchronous && GThreadPool != NULL )
	{
		// Queue this to our worker thread(s) for resolving
		GThreadPool->AddQueuedWork(&InCommand);
		CommandQueue.Add(&InCommand);
		return ELocalizationServiceOperationCommandResult::Succeeded;
	}
	else
	{
		InCommand.bCommandSuccessful = InCommand.DoWork();

		InCommand.Worker->UpdateStates();

		OutputCommandMessages(InCommand);

		// Callback now if present. When asynchronous, this callback gets called from Tick().
		ELocalizationServiceOperationCommandResult::Type Result = InCommand.bCommandSuccessful ? ELocalizationServiceOperationCommandResult::Succeeded : ELocalizationServiceOperationCommandResult::Failed;
		InCommand.OperationCompleteDelegate.ExecuteIfBound(InCommand.Operation, Result);

		return Result;
	}
}


void FOneSkyLocalizationServiceProvider::ImportAllCulturesForTargetFromOneSky(TWeakObjectPtr<ULocalizationTarget> LocalizationTarget, bool bIsTargetSet)
{
	check(LocalizationTarget.IsValid());

	if (!bIsTargetSet)
	{
		GWarn->BeginSlowTask(LOCTEXT("ImportingFromLocalizationService", "Importing Latest from Localization Service..."), true);
	}

	FString EngineOrGamePath = LocalizationTarget->IsMemberOfEngineTargetSet() ? "Engine" : "Game";

	for (FCultureStatistics CultureStats : LocalizationTarget->Settings.SupportedCulturesStatistics)
	{
		ILocalizationServiceProvider& Provider = ILocalizationServiceModule::Get().GetProvider();
		TSharedRef<FDownloadLocalizationTargetFile, ESPMode::ThreadSafe> DownloadTargetFileOp = ILocalizationServiceOperation::Create<FDownloadLocalizationTargetFile>();
		DownloadTargetFileOp->SetInTargetGuid(LocalizationTarget->Settings.Guid);
		DownloadTargetFileOp->SetInLocale(CultureStats.CultureName);

		// Put the intermediary .po files in a temporary directory in Saved for now
		FString Path = FPaths::ProjectSavedDir() / "Temp" / EngineOrGamePath / LocalizationTarget->Settings.Name / CultureStats.CultureName / LocalizationTarget->Settings.Name + ".po";
		FPaths::MakePathRelativeTo(Path, *FPaths::ProjectDir());
		DownloadTargetFileOp->SetInRelativeOutputFilePathAndName(Path);

		FilesDownloadingForImportFromOneSky.Add(Path);
		IPlatformFile &PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		// Delete this file if it exists so we don't accidentally import old data
		PlatformFile.DeleteFile(*Path);

		auto OperationCompleteDelegate = FLocalizationServiceOperationComplete::CreateRaw(this, &FOneSkyLocalizationServiceProvider::ImportCultureForTargetFromOneSky_Callback, bIsTargetSet);

		Provider.Execute(DownloadTargetFileOp, TArray<FLocalizationServiceTranslationIdentifier>(), ELocalizationServiceOperationConcurrency::Asynchronous, OperationCompleteDelegate);
	}
}



void FOneSkyLocalizationServiceProvider::ImportCultureForTargetFromOneSky_Callback(const FLocalizationServiceOperationRef& Operation, ELocalizationServiceOperationCommandResult::Type Result, bool bIsTargetSet)
{
	TSharedPtr<FDownloadLocalizationTargetFile, ESPMode::ThreadSafe> DownloadLocalizationTargetOp = StaticCastSharedRef<FDownloadLocalizationTargetFile>(Operation);
	bool bError = !(Result == ELocalizationServiceOperationCommandResult::Succeeded);
	FText ErrorText = FText::GetEmpty();
	FGuid InTargetGuid;
	FString InLocale;
	FString InRelativeOutputFilePathAndName;
	FString AbsoluteFilePathAndName;
	FString TargetName;
	ULocalizationTarget* Target = nullptr;
	bool bFinishedDownloading = false;
	if (DownloadLocalizationTargetOp.IsValid())
	{
		ErrorText = DownloadLocalizationTargetOp->GetOutErrorText();

		InTargetGuid = DownloadLocalizationTargetOp->GetInTargetGuid();
		InLocale = DownloadLocalizationTargetOp->GetInLocale();
		InRelativeOutputFilePathAndName = DownloadLocalizationTargetOp->GetInRelativeOutputFilePathAndName();

		TargetName = FPaths::GetBaseFilename(InRelativeOutputFilePathAndName);
		FString EngineOrGamePath = FPaths::GetBaseFilename(FPaths::GetPath(FPaths::GetPath(FPaths::GetPath(InRelativeOutputFilePathAndName))));
		bool bIsEngineTarget = EngineOrGamePath == "Engine";
		Target = ILocalizationModule::Get().GetLocalizationTargetByName(TargetName, bIsEngineTarget);

		// Remove each file we get a callback for so we know when we've gotten a callback for all of them
		FilesDownloadingForImportFromOneSky.Remove(InRelativeOutputFilePathAndName);

		int32 TotalNumber = 0;
		if (bIsTargetSet)
		{
			for (ULocalizationTarget* LocalizationTarget : Target->GetOuterULocalizationTargetSet()->TargetObjects)
			{
				TotalNumber += LocalizationTarget->Settings.SupportedCulturesStatistics.Num();
			}
		}
		else
		{
			TotalNumber = Target->Settings.SupportedCulturesStatistics.Num();
		}

		// Update progress bar
		GWarn->StatusUpdate(TotalNumber - FilesDownloadingForImportFromOneSky.Num(),
			TotalNumber,
			LOCTEXT("DownloadingFilesFromLocalizationService", "Downloading Files from Localization Service..."));

		AbsoluteFilePathAndName = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir() / InRelativeOutputFilePathAndName);

		// Once we have gotten the callback for each file for this import, then start importing
		if (FilesDownloadingForImportFromOneSky.Num() == 0)
		{
			GWarn->StatusUpdate(100, 100, LOCTEXT("ImportFromLocalizationServiceFinished", "Import from Localization Service Complete!"));
			GWarn->EndSlowTask();
			bFinishedDownloading = true;
		}
	}
	if (!bError && ErrorText.IsEmpty())
	{
		if (!DownloadLocalizationTargetOp.IsValid())
		{
			bError = true;
		}

		if ( !InRelativeOutputFilePathAndName.IsEmpty())
		{
			if (!FPaths::FileExists(AbsoluteFilePathAndName))
			{
				bError = true;
			}
		}
		else
		{
			bError = true;
		}

		if (bError)
		{
			if (ErrorText.IsEmpty())
			{
				ErrorText = LOCTEXT("DownloadLatestFromLocalizationServiceFileProcessError", "An error occured when processing the file downloaded from the Localization Service.");
			}
		}
	}
	else
	{
		bError = true;
		if (ErrorText.IsEmpty())
		{
			ErrorText = LOCTEXT("DownloadLatestFromLocalizationServiceDownloadError", "An error occured while downloading the file from the Localization Service.");
		}
	}

	if (bError || !ErrorText.IsEmpty())
	{
		if (ErrorText.IsEmpty())
		{
			ErrorText = LOCTEXT("DownloadLatestFromLocalizationServiceUnspecifiedError", "An unspecified error occured when trying download and import from the Localization Service.");
		}

		FText ErrorNotify = FText::Format(LOCTEXT("ImportLatestForAllCulturesForTargetFromOneSkyFail", "{0} translations for {1} target failed to import from OneSky!"), FText::FromString(InLocale), FText::FromString(TargetName));
		FMessageLog OneSkyMessageLog("OneSky");
		OneSkyMessageLog.Error(ErrorNotify);
		OneSkyMessageLog.Error(ErrorText);
		OneSkyMessageLog.Notify(ErrorNotify);
	}

	if (bFinishedDownloading)
	{
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
		if (bIsTargetSet)
		{
			ULocalizationTargetSet* TargetSet = Target->GetOuterULocalizationTargetSet();
			LocalizationCommandletTasks::ImportTextForTargets(MainFrameParentWindow.ToSharedRef(), TargetSet->TargetObjects, FPaths::GetPath(FPaths::GetPath(FPaths::GetPath(AbsoluteFilePathAndName))));
		}
		else
		{
			LocalizationCommandletTasks::ImportTextForTarget(MainFrameParentWindow.ToSharedRef(), Target, FPaths::GetPath(FPaths::GetPath(AbsoluteFilePathAndName)));
		}
	}
}

void FOneSkyLocalizationServiceProvider::ExportAllCulturesForTargetToOneSky(TWeakObjectPtr<ULocalizationTarget> LocalizationTarget, bool bIsTargetSet)
{
	check(LocalizationTarget.IsValid());

	// If this is only one target, not a whole set, get confirmation and do export here (otherwise this is handled in the calling function)
	if (!bIsTargetSet)
	{
		bool bAccepted = GWarn->YesNof(FText::Format(
			LOCTEXT("ExportAllCulturesForTargetToOneSkyConfirm", "All data in OneSky for target {0} will be overwritten with your local copy!\nThis cannot be undone.\nAre you sure you want to export all cultures for this target to OneSky?"),
			FText::FromString(LocalizationTarget->Settings.Name)));
		
		if (!bAccepted)
		{
			return;
		}

		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
		FString EngineOrGamePath = LocalizationTarget->IsMemberOfEngineTargetSet() ? "Engine" : "Game";
		FString AbsoluteFolderPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir() / "Temp" / EngineOrGamePath / LocalizationTarget->Settings.Name);
		
		// Delete old files if they exists so we don't accidentally export old data
		IPlatformFile &PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		PlatformFile.DeleteDirectoryRecursively(*AbsoluteFolderPath);

		// Export to file
		LocalizationCommandletTasks::ExportTextForTarget(MainFrameParentWindow.ToSharedRef(), LocalizationTarget.Get(), AbsoluteFolderPath);

		GWarn->BeginSlowTask(LOCTEXT("ExportingToOneSky", "Exporting Latest to OneSky..."), true);
	}

	for (FCultureStatistics CultureStats : LocalizationTarget->Settings.SupportedCulturesStatistics)
	{
		ILocalizationServiceProvider& Provider = ILocalizationServiceModule::Get().GetProvider();
		TSharedRef<FUploadLocalizationTargetFile, ESPMode::ThreadSafe> UploadFileOp = ILocalizationServiceOperation::Create<FUploadLocalizationTargetFile>();
		UploadFileOp->SetInTargetGuid(LocalizationTarget->Settings.Guid);
		UploadFileOp->SetInLocale(CultureStats.CultureName);
		FString EngineOrGamePath = LocalizationTarget->IsMemberOfEngineTargetSet() ? "Engine" : "Game";

		// Put the intermediary .po files in a temporary directory in Saved for now
		FString Path = FPaths::ProjectSavedDir() / "Temp" / EngineOrGamePath / LocalizationTarget->Settings.Name / CultureStats.CultureName / LocalizationTarget->Settings.Name + ".po";
		FPaths::MakePathRelativeTo(Path, *FPaths::ProjectDir());
		UploadFileOp->SetInRelativeInputFilePathAndName(Path);

		FilesUploadingForExportToOneSky.Add(Path);

		Provider.Execute(UploadFileOp, TArray<FLocalizationServiceTranslationIdentifier>(), ELocalizationServiceOperationConcurrency::Asynchronous, 
			FLocalizationServiceOperationComplete::CreateRaw(this, &FOneSkyLocalizationServiceProvider::ExportCultureForTargetToOneSky_Callback, bIsTargetSet));
	}
}



void FOneSkyLocalizationServiceProvider::ExportCultureForTargetToOneSky_Callback(const FLocalizationServiceOperationRef& Operation, ELocalizationServiceOperationCommandResult::Type Result, bool bIsTargetSet)
{
	TSharedPtr<FUploadLocalizationTargetFile, ESPMode::ThreadSafe> UploadLocalizationTargetOp = StaticCastSharedRef<FUploadLocalizationTargetFile>(Operation);
	bool bError = !(Result == ELocalizationServiceOperationCommandResult::Succeeded);
	FText ErrorText = FText::GetEmpty();
	FGuid InTargetGuid;
	FString InRelativeInputFilePathAndName;
	FString TargetName = "";
	FString CultureName = "";
	ULocalizationTarget* Target = nullptr;
	if (UploadLocalizationTargetOp.IsValid())
	{
		InTargetGuid = UploadLocalizationTargetOp->GetInTargetGuid();
		CultureName = UploadLocalizationTargetOp->GetInLocale();
		InRelativeInputFilePathAndName = UploadLocalizationTargetOp->GetInRelativeInputFilePathAndName();

		TargetName = FPaths::GetBaseFilename(InRelativeInputFilePathAndName);
		FString EngineOrGamePath = FPaths::GetBaseFilename(FPaths::GetPath(FPaths::GetPath(FPaths::GetPath(InRelativeInputFilePathAndName))));
		bool bIsEngineTarget = EngineOrGamePath == "Engine";
		Target = ILocalizationModule::Get().GetLocalizationTargetByName(TargetName, bIsEngineTarget);

		// Remove each file we get a callback for so we know when we've gotten a callback for all of them
		FilesDownloadingForImportFromOneSky.Remove(InRelativeInputFilePathAndName);

		ErrorText = UploadLocalizationTargetOp->GetOutErrorText();

		FilesUploadingForExportToOneSky.Remove(InRelativeInputFilePathAndName);

		int32 TotalNumber = 0;
		if (bIsTargetSet)
		{
			for (ULocalizationTarget* LocalizationTarget : Target->GetOuterULocalizationTargetSet()->TargetObjects)
			{
				TotalNumber += LocalizationTarget->Settings.SupportedCulturesStatistics.Num();
			}
		}
		else
		{
			TotalNumber = Target->Settings.SupportedCulturesStatistics.Num();
		}

		// Update progress bar
		GWarn->StatusUpdate(TotalNumber - FilesUploadingForExportToOneSky.Num(),
			TotalNumber,
			LOCTEXT("UploadingFilestoLocalizationService", "Uploading Files to Localization Service..."));

		if (FilesUploadingForExportToOneSky.Num() == 0)
		{
			GWarn->EndSlowTask();
		}
	}

	// Try to get display name
	FInternationalization& I18N = FInternationalization::Get();
	FCulturePtr CulturePtr = I18N.GetCulture(CultureName);
	FString CultureDisplayName = CultureName;
	if (CulturePtr.IsValid())
	{
		CultureName = CulturePtr->GetDisplayName();
	}

	if (!bError && ErrorText.IsEmpty())
	{
		FText SuccessText = FText::Format(LOCTEXT("ExportTranslationsToTranslationServiceSuccess", "{0} translations for {1} target uploaded for processing to Translation Service."), FText::FromString(CultureDisplayName), FText::FromString(TargetName));
		FMessageLog TranslationEditorMessageLog("TranslationEditor");
		TranslationEditorMessageLog.Info(SuccessText);
		TranslationEditorMessageLog.Notify(SuccessText, EMessageSeverity::Info, true);
	}
	else
	{
		if (ErrorText.IsEmpty())
		{
			ErrorText = LOCTEXT("ExportToLocalizationServiceUnspecifiedError", "An unspecified error occured when trying to export to the Localization Service.");
		}

		FText ErrorNotify = FText::Format(LOCTEXT("SaveSelectedTranslationsToTranslationServiceFail", "{0} translations for {1} target failed to export to Translation Service!"), FText::FromString(CultureDisplayName), FText::FromString(TargetName));
		FMessageLog TranslationEditorMessageLog("TranslationEditor");
		TranslationEditorMessageLog.Error(ErrorNotify);
		TranslationEditorMessageLog.Error(ErrorText);
		TranslationEditorMessageLog.Notify(ErrorNotify);
	}
}

void FOneSkyLocalizationServiceProvider::ImportAllTargetsForTargetSetFromOneSky(TWeakObjectPtr<ULocalizationTargetSet> LocalizationTargetSet)
{
	check(LocalizationTargetSet.IsValid());

	GWarn->BeginSlowTask(LOCTEXT("ImportingFromLocalizationService", "Importing Latest from Localization Service..."), true);

	bool bIsTargetSet = true;
	for (ULocalizationTarget* LocalizationTarget : LocalizationTargetSet->TargetObjects)
	{
		ImportAllCulturesForTargetFromOneSky(LocalizationTarget, bIsTargetSet);
	}
}

void FOneSkyLocalizationServiceProvider::ExportAllTargetsForTargetSetToOneSky(TWeakObjectPtr<ULocalizationTargetSet> LocalizationTargetSet)
{
	// If this is only one target, not a whole set, get confirmation and do export here

	if (!LocalizationTargetSet.IsValid())
	{
		return;
	}

	if (LocalizationTargetSet->TargetObjects.Num() < 1)
	{
		return;
	}

	FString EngineOrGamePath = LocalizationTargetSet->TargetObjects[0]->IsMemberOfEngineTargetSet() ? "Engine" : "Game";

	bool bAccepted = GWarn->YesNof(
		FText::Format(LOCTEXT("ExportAllCulturesForTargetsToOneSkyConfirm", "All data in OneSky for the {0} set of Targets will be overwritten with your local copy!\nThis cannot be undone.\nAre you sure you want to export all cultures for all targets for this set of targets to OneSky?"),
		FText::FromString(EngineOrGamePath)));

	if (!bAccepted)
	{
		return;
	}

	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
	FString AbsoluteFolderPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir() / "Temp" / EngineOrGamePath / "");

	IPlatformFile &PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	// Delete old files if they exists so we don't accidentally export old data
	PlatformFile.DeleteDirectoryRecursively(*AbsoluteFolderPath);

	LocalizationCommandletTasks::ExportTextForTargets(MainFrameParentWindow.ToSharedRef(), LocalizationTargetSet->TargetObjects, AbsoluteFolderPath);

	GWarn->BeginSlowTask(LOCTEXT("ExportingToOneSky", "Exporting Latest to OneSky..."), true);

	bool bIsTargetSet = true;
	for (ULocalizationTarget* LocalizationTarget : LocalizationTargetSet->TargetObjects)
	{
		ExportAllCulturesForTargetToOneSky(LocalizationTarget, bIsTargetSet);
	}
}


#undef LOCTEXT_NAMESPACE
