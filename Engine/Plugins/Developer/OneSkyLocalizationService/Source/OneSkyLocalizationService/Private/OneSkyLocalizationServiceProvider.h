// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILocalizationServiceProvider.h"
#include "Internationalization/Culture.h"
#include "Containers/Queue.h"
#include "IOneSkyLocalizationServiceWorker.h"
#include "Misc/DateTime.h"

class FOneSkyLocalizationServiceState;
class FUICommandList;

class FOneSkyLocalizationServiceCommand;
class FToolBarBuilder;
class IDetailCategoryBuilder;
class ULocalizationTarget;
class ULocalizationTargetSet;

DECLARE_DELEGATE_RetVal(FOneSkyLocalizationServiceWorkerRef, FGetOneSkyLocalizationServiceWorker)

// Define how the TMap from FLocalizationServiceTranslationIdentifier works
template <typename ValueType>
struct FLocalizationServiceTranslationIdentifierKeyFuncs : BaseKeyFuncs < TPair< FLocalizationServiceTranslationIdentifier, ValueType>, FLocalizationServiceTranslationIdentifier, false >
{
	typedef BaseKeyFuncs <
		TPair<FLocalizationServiceTranslationIdentifier, ValueType>,
		FLocalizationServiceTranslationIdentifier
	> Super;
	typedef typename Super::ElementInitType ElementInitType;
	typedef typename Super::KeyInitType     KeyInitType;
	static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element.Key;
	}
	static FORCEINLINE bool Matches(const FLocalizationServiceTranslationIdentifier& A, const FLocalizationServiceTranslationIdentifier& B)
	{
		return A.Culture.IsValid() && B.Culture.IsValid() &&
			A.LocalizationTargetGuid == B.LocalizationTargetGuid &&
			A.Culture->GetName().Equals(B.Culture->GetName()) &&
			A.Namespace.Equals(B.Namespace, ESearchCase::CaseSensitive) &&
			A.Source.Equals(B.Source, ESearchCase::CaseSensitive);
	}
	static FORCEINLINE uint32 GetKeyHash(const FLocalizationServiceTranslationIdentifier& Key)
	{
		if (!Key.Culture.IsValid())
		{
			return 0;
		}

		uint32 GuidHash = GetTypeHash(Key.LocalizationTargetGuid);
		uint32 CultureHash = FCrc::StrCrc32<TCHAR>(*Key.Culture->GetName());
		uint32 NamespaceHash = FCrc::StrCrc32<TCHAR>(*Key.Namespace);
		uint32 SourceHash = FCrc::StrCrc32<TCHAR>(*Key.Source);

		return HashCombine(HashCombine(HashCombine(GuidHash, CultureHash), NamespaceHash), SourceHash);
	}
};

// Struct representing a show import task in our queue
struct FShowImportTaskQueueItem
{
	int32 ProjectId;
	int32 ImportId;
	FDateTime CreatedTimestamp;
	int32 ExecutionDelayInSeconds;
};

class FOneSkyLocalizationServiceProvider : public ILocalizationServiceProvider
{
public:
	/** Constructor */
	FOneSkyLocalizationServiceProvider()
		: bServerAvailable(false)
		, PersistentConnection(NULL)
	{
	}

	/* ILocalizationServiceProvider implementation */
	virtual void Init(bool bForceConnection = true) override;
	virtual void Close() override;
	virtual FText GetStatusText() const override;
	virtual bool IsEnabled() const override;
	virtual bool IsAvailable() const override;
	virtual const FName& GetName(void) const override;
	virtual const FText GetDisplayName() const override;
	virtual ELocalizationServiceOperationCommandResult::Type GetState(const TArray<FLocalizationServiceTranslationIdentifier>& InTranslationIds, TArray< TSharedRef<ILocalizationServiceState, ESPMode::ThreadSafe> >& OutState, ELocalizationServiceCacheUsage::Type InStateCacheUsage) override;
	virtual ELocalizationServiceOperationCommandResult::Type Execute(const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation, const TArray<FLocalizationServiceTranslationIdentifier>& InTranslationIds, ELocalizationServiceOperationConcurrency::Type InConcurrency = ELocalizationServiceOperationConcurrency::Synchronous, const FLocalizationServiceOperationComplete& InOperationCompleteDelegate = FLocalizationServiceOperationComplete()) override;
	virtual bool CanCancelOperation( const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation ) const override;
	virtual void CancelOperation( const TSharedRef<ILocalizationServiceOperation, ESPMode::ThreadSafe>& InOperation ) override;
	virtual void Tick() override;
#if LOCALIZATION_SERVICES_WITH_SLATE
	virtual void CustomizeSettingsDetails(IDetailCategoryBuilder& DetailCategoryBuilder) const override;
	virtual void CustomizeTargetDetails(IDetailCategoryBuilder& DetailCategoryBuilder, TWeakObjectPtr<ULocalizationTarget> LocalizationTarget) const override;
	virtual void CustomizeTargetToolbar(TSharedRef<FExtender>& MenuExtender, TWeakObjectPtr<ULocalizationTarget> LocalizationTarget) const override;
	virtual void CustomizeTargetSetToolbar(TSharedRef<FExtender>& MenuExtender, TWeakObjectPtr<ULocalizationTargetSet> ULocalizationTargetSet) const override;
#endif // LOCALIZATION_SERVICES_WITH_SLATE

	/**
	 * Register a worker with the provider.
	 * This is used internally so the provider can maintain a map of all available operations.
	 */
	void RegisterWorker( const FName& InName, const FGetOneSkyLocalizationServiceWorker& InDelegate );

	/** Helper function used to update state cache */
	TSharedRef<FOneSkyLocalizationServiceState, ESPMode::ThreadSafe> GetStateInternal(const FLocalizationServiceTranslationIdentifier& InTranslationId);

	/**
	 * Connects to the Localization service server if the persistent connection is not already established.
	 *
	 * @return true if the connection is established or became established and false if the connection failed.
	 */
	bool EstablishPersistentConnection();

	/** Get the persistent connection, if any */
	class FOneSkyConnection* GetPersistentConnection()
	{
		return PersistentConnection;
	}

	// Enqueue a show import task for later exection
	void EnqueShowImportTask(FShowImportTaskQueueItem QueueItem)
	{
		ShowImportTaskQueue.Enqueue(QueueItem);
	}

private:

	/** Helper function used to create a worker for a particular operation */
	TSharedPtr<class IOneSkyLocalizationServiceWorker, ESPMode::ThreadSafe> CreateWorker(const FName& InOperationName) const;

	/** 
	 * Logs any messages that a command needs to output.
	 */
	void OutputCommandMessages(const class FOneSkyLocalizationServiceCommand& InCommand) const;

	/**
	 * Helper function for running command 'synchronously'.
	 * This really doesn't execute synchronously; rather it adds the command to the queue & does not return until
	 * the command is completed.
	 */
	ELocalizationServiceOperationCommandResult::Type ExecuteSynchronousCommand(class FOneSkyLocalizationServiceCommand& InCommand, const FText& Task, bool bSuppressResponseMsg);

	/**
	 * Run a command synchronously or asynchronously.
	 */
	ELocalizationServiceOperationCommandResult::Type IssueCommand(class FOneSkyLocalizationServiceCommand& InCommand, const bool bSynchronous);

	/** Add the buttons to the toolbar for this localization target */
	void AddTargetToolbarButtons(FToolBarBuilder& ToolbarBuilder, TWeakObjectPtr<ULocalizationTarget> InLocalizationTarget, TSharedRef< FUICommandList > CommandList);

	/** Add the buttons to the toolbar for this set of localization targets */
	void AddTargetSetToolbarButtons(FToolBarBuilder& ToolbarBuilder, TWeakObjectPtr<ULocalizationTargetSet> InLocalizationTargetSet, TSharedRef< FUICommandList > CommandList);

	/** Download and import all translations for all cultures for the specified target from OneSky */
	void ImportAllCulturesForTargetFromOneSky(TWeakObjectPtr<ULocalizationTarget> LocalizationTarget, bool bIsTargetSet);

	/** Called when done downloading localization data for culture for a target from OneSky */
	void ImportCultureForTargetFromOneSky_Callback(const FLocalizationServiceOperationRef& Operation, ELocalizationServiceOperationCommandResult::Type Result, bool yes);

	/** Export and upload all cultures for a localization target to OneSky */
	void ExportAllCulturesForTargetToOneSky(TWeakObjectPtr<ULocalizationTarget> LocalizationTarget, bool bIsTargetSet);

	/** Called when done uploading localization data for a culture for a target from OnSky*/
	void ExportCultureForTargetToOneSky_Callback(const FLocalizationServiceOperationRef& Operation, ELocalizationServiceOperationCommandResult::Type Result, bool bIsTargetSet);

	/** Download and import all translations for all cultures for all targets for the specified target set from OneSky */
	void ImportAllTargetsForTargetSetFromOneSky(TWeakObjectPtr<ULocalizationTargetSet> LocalizationTargetSet);

	/** Export and upload all cultures for all targets for a localization target set to OneSky */
	void ExportAllTargetsForTargetSetToOneSky(TWeakObjectPtr<ULocalizationTargetSet> LocalizationTargetSet);

private:

	/** Indicates if Localization service integration is available or not. */
	bool bServerAvailable;

	/** A pointer to the persistent P4 connection for synchronous operations */
	class FOneSkyConnection* PersistentConnection;
	
	/** State cache */
	TMap<FLocalizationServiceTranslationIdentifier, TSharedRef<class FOneSkyLocalizationServiceState, ESPMode::ThreadSafe>, FDefaultSetAllocator, FLocalizationServiceTranslationIdentifierKeyFuncs<TSharedRef<class FOneSkyLocalizationServiceState, ESPMode::ThreadSafe>> >  StateCache;

	/** The currently registered Localization service operations */
	TMap<FName, FGetOneSkyLocalizationServiceWorker> WorkersMap;

	/** Queue for commands given by the main thread */
	TArray < FOneSkyLocalizationServiceCommand* > CommandQueue;

	/** Queue for import status tasks  */
	TQueue < FShowImportTaskQueueItem, EQueueMode::Mpsc > ShowImportTaskQueue;

	/** Array of file names being downloaded for import from OneSky */
	TArray < FString > FilesDownloadingForImportFromOneSky;

	/** Array of file names being uploaded for export into OneSky */
	TArray < FString > FilesUploadingForExportToOneSky;
};
