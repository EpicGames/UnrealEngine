// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OneSkyLocalizationServiceResponseTypes.generated.h"


// LIST PROJECT GROUPS

USTRUCT()
struct FOneSkyListProjectGroupsResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	UPROPERTY()
	int32 record_count;

	UPROPERTY()
	int32 page_count;

	UPROPERTY()
	FString next_page;

	UPROPERTY()
	FString prev_page;

	UPROPERTY()
	FString first_page;

	UPROPERTY()
	FString last_page;

	/** Default constructor. */
	FOneSkyListProjectGroupsResponseMeta()
		: status(-1)
		, record_count(-1)
		, page_count(-1)
		, next_page(TEXT(""))
		, prev_page(TEXT(""))
		, first_page(TEXT(""))
		, last_page(TEXT(""))
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListProjectGroupsResponseMeta(ENoInit) { }
};


USTRUCT()
struct FOneSkyListProjectGroupsResponseDataItem
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString id;

	UPROPERTY()
	FString name;

	/** Default constructor. */
	FOneSkyListProjectGroupsResponseDataItem()
		: id("-1")
		, name(TEXT(""))
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListProjectGroupsResponseDataItem(ENoInit) { }
};

/**
* Response from a List Project Groups query on OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/project_group.md
*/
USTRUCT()
struct FOneSkyListProjectGroupsResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyListProjectGroupsResponseMeta meta;

	UPROPERTY()
	TArray<FOneSkyListProjectGroupsResponseDataItem> data;

	/** Default constructor. */
	FOneSkyListProjectGroupsResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListProjectGroupsResponse(ENoInit) { }
};

// end LIST PROJECT GROUPS


// SHOW PROJECT GROUP

USTRUCT()
struct FOneSkyShowProjectGroupResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	/** Default constructor. */
	FOneSkyShowProjectGroupResponseMeta()
		: status(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyShowProjectGroupResponseMeta(ENoInit) { }
};

USTRUCT()
struct FOneSkyShowProjectGroupResponseData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString id;

	UPROPERTY()
	FString name;

	UPROPERTY()
	int32 description;

	UPROPERTY()
	FString project_count;

	/** Default constructor. */
	FOneSkyShowProjectGroupResponseData()
		: id("-1")
		, name(TEXT(""))
		, description(-1)
		, project_count("-1")
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyShowProjectGroupResponseData(ENoInit) { }
};

/**
* Response from a Show Project Group query on OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/project_group.md
*/
USTRUCT()
struct FOneSkyShowProjectGroupResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyShowProjectGroupResponseMeta meta;

	UPROPERTY()
	FOneSkyShowProjectGroupResponseData data;

	/** Default constructor. */
	FOneSkyShowProjectGroupResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyShowProjectGroupResponse(ENoInit) { }
};

// end SHOW PROJECT GROUP


// CREATE PROJECT GROUP

USTRUCT()
struct FOneSkyCreateProjectGroupResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	/** Default constructor. */
	FOneSkyCreateProjectGroupResponseMeta()
		: status(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyCreateProjectGroupResponseMeta(ENoInit) { }
};

USTRUCT()
struct FOneSkyCreateProjectGroupResponseBaseLanguage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString code;

	UPROPERTY()
	FString english_name;

	UPROPERTY()
	FString local_name;

	UPROPERTY()
	FString locale;

	UPROPERTY()
	FString region;

	/** Default constructor. */
	FOneSkyCreateProjectGroupResponseBaseLanguage()
		: code("")
		, english_name("")
		, local_name("")
		, locale("")
		, region("")
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyCreateProjectGroupResponseBaseLanguage(ENoInit) { }
};

USTRUCT()
struct FOneSkyCreateProjectGroupResponseData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 id;

	UPROPERTY()
	FString name;

	UPROPERTY()
	FOneSkyCreateProjectGroupResponseBaseLanguage base_language;

	/** Default constructor. */
	FOneSkyCreateProjectGroupResponseData()
		: id(-1)
		, name(TEXT(""))
		, base_language()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyCreateProjectGroupResponseData(ENoInit) { }
};

/**
* Response from a Create Project Group query on OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/project_group.md
*/
USTRUCT()
struct FOneSkyCreateProjectGroupResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyCreateProjectGroupResponseMeta meta;

	UPROPERTY()
	FOneSkyCreateProjectGroupResponseData data;

	/** Default constructor. */
	FOneSkyCreateProjectGroupResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyCreateProjectGroupResponse(ENoInit) { }
};

// end CREATE PROJECT GROUP


// LIST PROJECT GROUP LANGUAGES

USTRUCT()
struct FOneSkyListProjectGroupLanguagesResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	UPROPERTY()
	int32 record_count;

	/** Default constructor. */
	FOneSkyListProjectGroupLanguagesResponseMeta()
		: status(-1)
		, record_count(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListProjectGroupLanguagesResponseMeta(ENoInit) { }
};

USTRUCT()
struct FOneSkyListProjectGroupLanguagesResponseDataItem
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString code;

	UPROPERTY()
	FString english_name;

	UPROPERTY()
	FString local_name;

	UPROPERTY()
	FString locale;

	UPROPERTY()
	FString region;

	UPROPERTY()
	bool is_base_language;

	/** Default constructor. */
	FOneSkyListProjectGroupLanguagesResponseDataItem()
		: code("")
		, english_name("")
		, local_name("")
		, locale("")
		, region("")
		, is_base_language(false)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListProjectGroupLanguagesResponseDataItem(ENoInit) { }
};

/**
* Response from a List Project Group Languages query on OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/project_group.md
*/
USTRUCT()
struct FOneSkyListProjectGroupLanguagesResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyListProjectGroupLanguagesResponseMeta meta;

	UPROPERTY()
	TArray<FOneSkyListProjectGroupLanguagesResponseDataItem> data;

	/** Default constructor. */
	FOneSkyListProjectGroupLanguagesResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListProjectGroupLanguagesResponse(ENoInit) { }
};

// end LIST PROJECT GROUP LANGUAGES


// LIST PROJECTS IN GROUP

USTRUCT()
struct FOneSkyListProjectsInGroupResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	UPROPERTY()
	int32 record_count;

	/** Default constructor. */
	FOneSkyListProjectsInGroupResponseMeta()
		: status(-1)
		, record_count(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListProjectsInGroupResponseMeta(ENoInit) { }
};

USTRUCT()
struct FOneSkyListProjectsInGroupResponseDataItem
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 id;

	UPROPERTY()
	FString name;

	/** Default constructor. */
	FOneSkyListProjectsInGroupResponseDataItem()
		: id(-1)
		, name(TEXT(""))
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListProjectsInGroupResponseDataItem(ENoInit) { }
};

/**
* Response from a List Projects in Group query on OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/project.md
*/
USTRUCT()
struct FOneSkyListProjectsInGroupResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyListProjectsInGroupResponseMeta meta;

	UPROPERTY()
	TArray<FOneSkyListProjectsInGroupResponseDataItem> data;

	/** Default constructor. */
	FOneSkyListProjectsInGroupResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListProjectsInGroupResponse(ENoInit) { }
};

// end LIST PROJECTS IN GROUP


// SHOW PROJECT 

USTRUCT()
struct FOneSkyShowProjectResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	/** Default constructor. */
	FOneSkyShowProjectResponseMeta()
		: status(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyShowProjectResponseMeta(ENoInit) { }
};


USTRUCT()
struct FOneSkyShowProjectResponseProjectType
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString code;

	UPROPERTY()
	FString name;

	/** Default constructor. */
	FOneSkyShowProjectResponseProjectType()
		: code("")
		, name("")
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyShowProjectResponseProjectType(ENoInit) { }
};

USTRUCT()
struct FOneSkyShowProjectResponseData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 id;

	UPROPERTY()
	FString name;

	UPROPERTY()
	FString description;

	UPROPERTY()
	FOneSkyShowProjectResponseProjectType project_type;

	UPROPERTY()
	int32 string_count;

	UPROPERTY()
	int32 word_count;

	/** Default constructor. */
	FOneSkyShowProjectResponseData()
		: id(-1)
		, name(TEXT(""))
		, description("")
		, string_count(-1)
		, word_count(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyShowProjectResponseData(ENoInit) { }
};

/**
* Response from a Show Project query on OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/project.md
*/
USTRUCT()
struct FOneSkyShowProjectResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyShowProjectResponseMeta meta;

	UPROPERTY()
	FOneSkyShowProjectResponseData data;

	/** Default constructor. */
	FOneSkyShowProjectResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyShowProjectResponse(ENoInit) { }
};

// end SHOW PROJECT 


// CREATE PROJECT 

USTRUCT()
struct FOneSkyCreateProjectResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	/** Default constructor. */
	FOneSkyCreateProjectResponseMeta()
		: status(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyCreateProjectResponseMeta(ENoInit) { }
};


USTRUCT()
struct FOneSkyCreateProjectResponseProjectType
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString code;

	UPROPERTY()
	FString name;

	/** Default constructor. */
	FOneSkyCreateProjectResponseProjectType()
		: code("")
		, name("")
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyCreateProjectResponseProjectType(ENoInit) { }
};

USTRUCT()
struct FOneSkyCreateProjectResponseData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 id;

	UPROPERTY()
	FString name;

	UPROPERTY()
	FString description;

	UPROPERTY()
	FOneSkyShowProjectResponseProjectType project_type;

	/** Default constructor. */
	FOneSkyCreateProjectResponseData()
		: id(-1)
		, name(TEXT(""))
		, description("")
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyCreateProjectResponseData(ENoInit) { }
};

/**
* Response from a Create Project query on OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/project.md
*/
USTRUCT()
struct FOneSkyCreateProjectResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyCreateProjectResponseMeta meta;

	UPROPERTY()
	FOneSkyCreateProjectResponseData data;

	/** Default constructor. */
	FOneSkyCreateProjectResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyCreateProjectResponse(ENoInit) { }
};

// end CREATE PROJECT 


// LIST PROJECT LANGUAGES

USTRUCT()
struct FOneSkyListProjectLanguagesResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	UPROPERTY()
	int32 record_count;

	/** Default constructor. */
	FOneSkyListProjectLanguagesResponseMeta()
		: status(-1)
		, record_count(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListProjectLanguagesResponseMeta(ENoInit) { }
};

USTRUCT()
struct FOneSkyListProjectLanguagesResponseDataItem
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString code;

	UPROPERTY()
	FString english_name;

	UPROPERTY()
	FString local_name;

	UPROPERTY()
	FString locale;

	UPROPERTY()
	FString region;

	UPROPERTY()
	bool is_base_language;

	UPROPERTY()
	bool is_ready_to_publish;

	UPROPERTY()
	FString translation_progress;

	/** Default constructor. */
	FOneSkyListProjectLanguagesResponseDataItem()
		: code("")
		, english_name("")
		, local_name("")
		, locale("")
		, region("")
		, is_base_language(false)
		, is_ready_to_publish(false)
		, translation_progress("")
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListProjectLanguagesResponseDataItem(ENoInit) { }
};

/**
* Response from a List Project Languages query on OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/project.md
*/
USTRUCT()
struct FOneSkyListProjectLanguagesResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyListProjectLanguagesResponseMeta meta;

	UPROPERTY()
	TArray<FOneSkyListProjectLanguagesResponseDataItem> data;

	/** Default constructor. */
	FOneSkyListProjectLanguagesResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListProjectLanguagesResponse(ENoInit) { }
};

// end LIST PROJECT LANGUAGES


// TRANSLATION STATUS

USTRUCT()
struct FOneSkyTranslationStatusResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	/** Default constructor. */
	FOneSkyTranslationStatusResponseMeta()
		: status(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyTranslationStatusResponseMeta(ENoInit) { }
};


USTRUCT()
struct FOneSkyTranslationStatusResponseLocale
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString code;

	UPROPERTY()
	FString english_name;

	UPROPERTY()
	FString local_name;

	UPROPERTY()
	FString locale;

	UPROPERTY()
	FString region;

	/** Default constructor. */
	FOneSkyTranslationStatusResponseLocale()
		: code("")
		, english_name("")
		, local_name("")
		, locale("")
		, region("")
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyTranslationStatusResponseLocale(ENoInit) { }
};

USTRUCT()
struct FOneSkyTranslationStatusResponseData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString file_name;

	UPROPERTY()
	FOneSkyTranslationStatusResponseLocale locale;

	UPROPERTY()
	FString progress;

	UPROPERTY()
	int32 string_count;

	UPROPERTY()
	int32 word_count;

	/** Default constructor. */
	FOneSkyTranslationStatusResponseData()
		: file_name("")
		, locale()
		, progress("")
		, string_count(-1)
		, word_count(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyTranslationStatusResponseData(ENoInit) { }
};

/**
* Response from a Translation Status query on OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/translation.md#status---translations-status
*/
USTRUCT()
struct FOneSkyTranslationStatusResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyTranslationStatusResponseMeta meta;

	UPROPERTY()
	FOneSkyTranslationStatusResponseData data;

	/** Default constructor. */
	FOneSkyTranslationStatusResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyTranslationStatusResponse(ENoInit) { }
};

// end TRANSLATION STATUS


// LIST UPLOADED FILES

USTRUCT()
struct FOneSkyListUploadedFilesResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	UPROPERTY()
	int32 record_count;

	UPROPERTY()
	int32 page_count;

	UPROPERTY()
	FString next_page;

	UPROPERTY()
	FString prev_page;

	UPROPERTY()
	FString first_page;

	UPROPERTY()
	FString last_page;

	/** Default constructor. */
	FOneSkyListUploadedFilesResponseMeta()
		: status(-1)
		, record_count(-1)
		, page_count(-1)
		, next_page(TEXT(""))
		, prev_page(TEXT(""))
		, first_page(TEXT(""))
		, last_page(TEXT(""))
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListUploadedFilesResponseMeta(ENoInit) { }
};

USTRUCT()
struct FOneSkyListUploadedFilesResponseLastImport
{
	GENERATED_USTRUCT_BODY()

		UPROPERTY()
	int32 id;

	UPROPERTY()
	FString status;

	/** Default constructor. */
	FOneSkyListUploadedFilesResponseLastImport()
		: id(-1)
		, status("")
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListUploadedFilesResponseLastImport(ENoInit) { }
};


USTRUCT()
struct FOneSkyListUploadedFilesResponseDataItem
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString file_name;

	UPROPERTY()
	int32 string_count;

	UPROPERTY()
	FOneSkyListUploadedFilesResponseLastImport last_import;

	UPROPERTY()
	FString uploaded_at;

	UPROPERTY()
	int32 uploaded_at_timestamp;

	/** Default constructor. */
	FOneSkyListUploadedFilesResponseDataItem()
		: file_name("")
		, string_count(-1)
		, last_import()
		, uploaded_at("")
		, uploaded_at_timestamp(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListUploadedFilesResponseDataItem(ENoInit) { }
};

/**
* Response from a List Uploaded Files query on OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/file.md
*/
USTRUCT()
struct FOneSkyListUploadedFilesResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyListUploadedFilesResponseMeta meta;

	UPROPERTY()
	TArray<FOneSkyListUploadedFilesResponseDataItem> data;

	/** Default constructor. */
	FOneSkyListUploadedFilesResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListUploadedFilesResponse(ENoInit) { }
};

// end LIST UPLOADED FILES

// UPLOAD FILE

USTRUCT()
struct FOneSkyUploadFileResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	/** Default constructor. */
	FOneSkyUploadFileResponseMeta()
		: status(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyUploadFileResponseMeta(ENoInit) { }
};


USTRUCT()
struct FOneSkyUploadFileResponseLanguage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString code;

	UPROPERTY()
	FString english_name;

	UPROPERTY()
	FString local_name;

	UPROPERTY()
	FString locale;

	UPROPERTY()
	FString region;

	/** Default constructor. */
	FOneSkyUploadFileResponseLanguage()
		: code("")
		, english_name("")
		, local_name("")
		, locale("")
		, region("")
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyUploadFileResponseLanguage(ENoInit) { }
};

USTRUCT()
struct FOneSkyUploadedFileResponseImport
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 id;

	UPROPERTY()
	FString created_at;

	UPROPERTY()
	int32 created_at_timestamp;

	/** Default constructor. */
	FOneSkyUploadedFileResponseImport()
		: id(-1)
		, created_at("")
		, created_at_timestamp(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyUploadedFileResponseImport(ENoInit) { }
};

USTRUCT()
struct FOneSkyUploadFileResponseData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString name;

	UPROPERTY()
	FString format;

	UPROPERTY()
	FOneSkyUploadFileResponseLanguage language;

	UPROPERTY()
	FOneSkyUploadedFileResponseImport import;

	/** Default constructor. */
	FOneSkyUploadFileResponseData()
		: name("")
		, format("")
		, language()
		, import()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyUploadFileResponseData(ENoInit) { }
};

/**
* Response from an Upload File post to OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/file.md
*/
USTRUCT()
struct FOneSkyUploadFileResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyUploadFileResponseMeta meta;

	UPROPERTY()
	FOneSkyUploadFileResponseData data;

	/** Default constructor. */
	FOneSkyUploadFileResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyUploadFileResponse(ENoInit) { }
};

// end UPLOAD FILE


// LIST PHRASE COLLECTIONS

USTRUCT()
struct FOneSkyListPhraseCollectionsResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	UPROPERTY()
	int32 record_count;

	UPROPERTY()
	int32 page_count;

	UPROPERTY()
	FString next_page;

	UPROPERTY()
	FString prev_page;

	UPROPERTY()
	FString first_page;

	UPROPERTY()
	FString last_page;

	/** Default constructor. */
	FOneSkyListPhraseCollectionsResponseMeta()
		: status(-1)
		, record_count(-1)
		, page_count(-1)
		, next_page(TEXT(""))
		, prev_page(TEXT(""))
		, first_page(TEXT(""))
		, last_page(TEXT(""))
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListPhraseCollectionsResponseMeta(ENoInit) { }
};

USTRUCT()
struct FOneSkyListPhraseCollectionsResponseDataItem
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString key;

	UPROPERTY()
	FString created_at;

	UPROPERTY()
	int32 created_at_timestamp;

	/** Default constructor. */
	FOneSkyListPhraseCollectionsResponseDataItem()
		: key("")
		, created_at("")
		, created_at_timestamp(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListPhraseCollectionsResponseDataItem(ENoInit) { }
};

/**
* Response from a List Uploaded Files query on OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/file.md
*/
USTRUCT()
struct FOneSkyListPhraseCollectionsResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyListPhraseCollectionsResponseMeta meta;

	UPROPERTY()
	TArray<FOneSkyListPhraseCollectionsResponseDataItem> data;

	/** Default constructor. */
	FOneSkyListPhraseCollectionsResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyListPhraseCollectionsResponse(ENoInit) { }
};

// end LIST PHRASE COLLECTIONS


// SHOW IMPORT TASK 

USTRUCT()
struct FOneSkyShowImportTaskResponseMeta
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 status;

	/** Default constructor. */
	FOneSkyShowImportTaskResponseMeta()
		: status(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyShowImportTaskResponseMeta(ENoInit) { }
};

USTRUCT()
struct FOneSkyShowImportTaskResponseLocale
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString code;

	UPROPERTY()
	FString english_name;

	UPROPERTY()
	FString local_name;

	UPROPERTY()
	FString locale;

	UPROPERTY()
	FString region;

	/** Default constructor. */
	FOneSkyShowImportTaskResponseLocale()
		: code("")
		, english_name("")
		, local_name("")
		, locale("")
		, region("")
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyShowImportTaskResponseLocale(ENoInit) { }
};


USTRUCT()
struct FOneSkyShowImportTaskResponseFile
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString name;

	UPROPERTY()
	FString format;

	UPROPERTY()
	FOneSkyShowImportTaskResponseLocale locale;

	/** Default constructor. */
	FOneSkyShowImportTaskResponseFile()
		: name("")
		, format("")
		, locale()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyShowImportTaskResponseFile(ENoInit) { }
};

USTRUCT()
struct FOneSkyShowImportTaskResponseData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	int32 id;

	UPROPERTY()
	FOneSkyShowImportTaskResponseFile file;

	UPROPERTY()
	int32 string_count;

	UPROPERTY()
	int32 word_count;

	UPROPERTY()
	FString status;

	UPROPERTY()
	FString created_at;

	UPROPERTY()
	int32 created_at_timestamp;

	/** Default constructor. */
	FOneSkyShowImportTaskResponseData()
		: id(-1)
		, file()
		, string_count(-1)
		, word_count(-1)
		, status("")
		, created_at("")
		, created_at_timestamp(-1)
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyShowImportTaskResponseData(ENoInit) { }
};

/**
* Response from a Show Import Task query on OneSky
* //https://github.com/onesky/api-documentation-platform/blob/master/resources/import_task.md
*/
USTRUCT()
struct FOneSkyShowImportTaskResponse
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FOneSkyShowImportTaskResponseMeta meta;

	UPROPERTY()
	FOneSkyShowImportTaskResponseData data;

	/** Default constructor. */
	FOneSkyShowImportTaskResponse()
		: meta()
		, data()
	{ }

	/** Creates an uninitialized instance. */
	FOneSkyShowImportTaskResponse(ENoInit) { }
};

// end SHOW IMPORT TASK 
