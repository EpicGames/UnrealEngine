// Copyright Epic Games, Inc. All Rights Reserved.

#include "OneSkyLocalizationServiceRevision.h"

#define LOCTEXT_NAMESPACE "OneSkyLocalizationService"

int32 FOneSkyLocalizationServiceRevision::GetRevisionNumber() const
{
	return RevisionNumber;
}

const FString& FOneSkyLocalizationServiceRevision::GetRevision() const
{
	return Revision;
}

const FString& FOneSkyLocalizationServiceRevision::GetUserName() const
{
	return UserName;
}

const FDateTime& FOneSkyLocalizationServiceRevision::GetDate() const
{
	return Date;
}

#undef LOCTEXT_NAMESPACE
