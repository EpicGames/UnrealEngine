// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILocalizationServiceRevision.h"
#include "Misc/DateTime.h"

class FOneSkyLocalizationServiceRevision : public ILocalizationServiceRevision
{
public:
	FOneSkyLocalizationServiceRevision()
		: RevisionNumber(0)
		, Date(0)
	{
	}

	/** ILocalizationServiceRevision interface */
	virtual int32 GetRevisionNumber() const override;
	virtual const FString& GetRevision() const override;
	virtual const FString& GetUserName() const override;
	virtual const FDateTime& GetDate() const override;

public:

	/** The revision number of this file */
	int32 RevisionNumber;

	/** The revision to display to the user */
	FString Revision;

	/** THe user that made the change for this revision */
	FString UserName;

	/** The date of this revision */
	FDateTime Date;
};
