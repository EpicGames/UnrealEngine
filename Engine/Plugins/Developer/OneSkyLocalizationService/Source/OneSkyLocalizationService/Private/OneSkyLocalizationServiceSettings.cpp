// Copyright Epic Games, Inc. All Rights Reserved.

#include "OneSkyLocalizationServiceSettings.h"
#include "Misc/FileHelper.h"
#include "Misc/ConfigCacheIni.h"
#include "Serialization/JsonSerializer.h"
#include "ILocalizationServiceModule.h"
#include "LocalizationServiceHelpers.h"
//#include "JsonInternationalizationManifestSerializer.h"


namespace OneSkySettingsConstants
{

/** The section of the ini file we load our settings from */
static const FString SettingsSection = TEXT("OneSkyLocalizationService.OneSkyLocalizationServiceSettings");

}

UOneSkyLocalizationTargetSettings::UOneSkyLocalizationTargetSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}


const FString& FOneSkyLocalizationServiceSettings::GetConnectionName() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return ConnectionInfo.Name;
}

void FOneSkyLocalizationServiceSettings::LoadSettings()
{
	FScopeLock ScopeLock(&CriticalSection);
	const FString& IniFile = LocalizationServiceHelpers::GetSettingsIni();

	GConfig->GetString(*OneSkySettingsConstants::SettingsSection, TEXT("ConnectionName"), ConnectionInfo.Name, IniFile);
	GConfig->GetString(*OneSkySettingsConstants::SettingsSection, TEXT("ConnectionApiKey"), ConnectionInfo.ApiKey, IniFile);
	GConfig->GetString(*OneSkySettingsConstants::SettingsSection, TEXT("ConnectionApiSecret"), ConnectionInfo.ApiSecret, IniFile);

	bSaveSecretKey = !ConnectionInfo.ApiSecret.IsEmpty();

	// If connection info was not loaded from .ini, try loading from JSON file
	if (ConnectionInfo.ApiKey.IsEmpty() && ConnectionInfo.ApiSecret.IsEmpty())
	{
		TArray<FString> CredentialsFileNames;
		// read json config file
		const FString CredentialsExtension = TEXT("credentials");
		FString CredentialsFolderPath = FPaths::EnginePluginsDir() / "Developer" / "OneSkyLocalizationService" / "Credentials";
		// Search for .manifest files 1 directory deep
		const FString DirectoryToSearchWildcard = FString::Printf(TEXT("%s/*"), *CredentialsFolderPath);
		TArray<FString> CredentialsFolders;
		CredentialsFolders.AddUnique(CredentialsFolderPath);
		IFileManager::Get().FindFiles(CredentialsFolders, *DirectoryToSearchWildcard, false, true);
		for (FString CredentialsFolder : CredentialsFolders)
		{
			// Add root back on
			CredentialsFolder = CredentialsFolderPath / CredentialsFolder;
			const FString SubFolderCredentialFileWildcard = FString::Printf(TEXT("%s/*.%s"), *CredentialsFolder, *CredentialsExtension);

			TArray<FString> CurrentFolderCredentialsFileNames;
			IFileManager::Get().FindFiles(CurrentFolderCredentialsFileNames, *SubFolderCredentialFileWildcard, true, false);

			for (FString& CurrentFolderCredentialsFileName : CurrentFolderCredentialsFileNames)
			{
				// Add root back on
				CurrentFolderCredentialsFileName = CredentialsFolder / CurrentFolderCredentialsFileName;
				CredentialsFileNames.AddUnique(CurrentFolderCredentialsFileName);
			}
		}

		for (FString CredentialsFile : CredentialsFileNames)
		{
			//read in file as string
			FString FileContents;
			if (!FFileHelper::LoadFileToString(FileContents, *CredentialsFile))
			{
				UE_LOGF(LogLocalizationService, Log, "Failed to load OneSky credentials file %ls.", *CredentialsFile);
				continue;
			}

			//parse as JSON
			TSharedPtr<FJsonObject> JSONObject;

			TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(FileContents);

			if (!FJsonSerializer::Deserialize(Reader, JSONObject) || !JSONObject.IsValid())
			{
				UE_LOGF(LogLocalizationService, Log, "Invalid JSON in OneSky credentials file %ls.", *CredentialsFile);
				continue;
			}

			TArray<TSharedPtr<FJsonValue> > CredentialsArray = JSONObject->GetArrayField(TEXT("Credentials"));

			for (TSharedPtr<FJsonValue> CredentialsItem : CredentialsArray)
			{
				bool bFoundCredentials = false;
				const TSharedPtr<FJsonObject>& CredentialsObject = CredentialsItem->AsObject();
				if (CredentialsObject->HasTypedField<EJson::String>(TEXT("Name")))
				{
					ConnectionInfo.Name = CredentialsObject->GetStringField(TEXT("Name"));
				}

				if (CredentialsObject->HasTypedField<EJson::String>(TEXT("ApiKey")))
				{
					ConnectionInfo.ApiKey = CredentialsObject->GetStringField(TEXT("ApiKey"));
					bFoundCredentials = true;
				}
				else
				{
					UE_LOGF(LogLocalizationService, Log, "Credentials file %ls is missing ApiKey for connection name %ls", *CredentialsFile, *ConnectionInfo.Name);
				}

				if (CredentialsObject->HasTypedField<EJson::String>(TEXT("ApiSecret")))
				{
					ConnectionInfo.ApiSecret = CredentialsObject->GetStringField(TEXT("ApiSecret"));
				}
				else
				{
					UE_LOGF(LogLocalizationService, Log, "Credentials file %ls is missing ApiSecret for connection name %ls", *CredentialsFile, *ConnectionInfo.Name);
				}

				if (bFoundCredentials)
				{
					break;
				}
			}
		}
	}
}

void FOneSkyLocalizationServiceSettings::SaveSettings() const
{
	FScopeLock ScopeLock(&CriticalSection);
	const FString& IniFile = LocalizationServiceHelpers::GetSettingsIni();
	GConfig->SetString(*OneSkySettingsConstants::SettingsSection, TEXT("ConnectionName"), *ConnectionInfo.Name, IniFile);
	GConfig->SetString(*OneSkySettingsConstants::SettingsSection, TEXT("ConnectionApiKey"), *ConnectionInfo.ApiKey, IniFile);

	// Only save secret key if asked to, otherwise store null string
	FString SecretKeyToSave = ConnectionInfo.ApiSecret;
	if (!bSaveSecretKey)
	{
		SecretKeyToSave = "";
	}

	GConfig->SetString(*OneSkySettingsConstants::SettingsSection, TEXT("ConnectionApiSecret"), *SecretKeyToSave, IniFile);
}

FOneSkyConnectionInfo FOneSkyLocalizationServiceSettings::GetConnectionInfo() const
{
	check(IsInGameThread());
	FOneSkyConnectionInfo OutConnectionInfo = ConnectionInfo;

	return OutConnectionInfo;
}

FOneSkyLocalizationTargetSetting* FOneSkyLocalizationServiceSettings::GetSettingsForTarget(FGuid TargetGuid, bool bCreateIfNotFound /* = false */)
{
	FScopeLock ScopeLock(&CriticalSection);
	TargetSettingsObject->LoadConfig();

	FOneSkyLocalizationTargetSetting* Settings = TargetSettingsObject->TargetSettings.FindByPredicate([&](const FOneSkyLocalizationTargetSetting& Setting)
	{
		return Setting.TargetGuid == TargetGuid;
	});

	if (!Settings && bCreateIfNotFound)
	{
		FOneSkyLocalizationTargetSetting NewSettings;
		NewSettings.TargetGuid = TargetGuid;
		int32 NewIndex = TargetSettingsObject->TargetSettings.Add(NewSettings);
		Settings = &(TargetSettingsObject->TargetSettings[NewIndex]);
	}

	return Settings;
}

void FOneSkyLocalizationServiceSettings::SetSettingsForTarget(FGuid TargetGuid, int32 ProjectId, FString FileName)
{
	FScopeLock ScopeLock(&CriticalSection);
	FOneSkyLocalizationTargetSetting* Settings = TargetSettingsObject->TargetSettings.FindByPredicate([&](const FOneSkyLocalizationTargetSetting& Setting)
	{
		return Setting.TargetGuid == TargetGuid;
	});

	if (!Settings)
	{
		FOneSkyLocalizationTargetSetting NewSettings;
		NewSettings.TargetGuid = TargetGuid;
		int32 NewIndex = TargetSettingsObject->TargetSettings.Add(NewSettings);
		Settings = &(TargetSettingsObject->TargetSettings[NewIndex]);
	}

	Settings->OneSkyProjectId = ProjectId;
	Settings->OneSkyFileName = FileName;

	TargetSettingsObject->SaveConfig();
}
