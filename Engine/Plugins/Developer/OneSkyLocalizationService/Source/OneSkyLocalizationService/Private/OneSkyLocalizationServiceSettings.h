// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OneSkyConnectionInfo.h"
#include "OneSkyLocalizationServiceSettings.generated.h"

/**
* Holds the OneSky settings for a Localization Target.
*/
USTRUCT()
struct FOneSkyLocalizationTargetSetting
{
	GENERATED_USTRUCT_BODY()

		FOneSkyLocalizationTargetSetting() : OneSkyProjectId(INDEX_NONE) {}

	/**
	* The GUID of the LocalizationTarget that these OneSky settings are for
	*/
	UPROPERTY()
	FGuid TargetGuid;

	/**
	* The id of the OneSky Project  this target belongs to
	*/
	UPROPERTY()
	int32 OneSkyProjectId;

	/**
	* The name of the file that corresponds to this target
	*/
	UPROPERTY()
	FString OneSkyFileName;

};

UCLASS(config = LocalizationServiceSettings)
class UOneSkyLocalizationTargetSettings : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/**
	* The array of settings for each target of this project 
	*/
	UPROPERTY(config)
	TArray<struct FOneSkyLocalizationTargetSetting> TargetSettings;

};

class FOneSkyLocalizationServiceSettings
{
public:

	FOneSkyLocalizationServiceSettings() : bSaveSecretKey(false) 
	{
		TargetSettingsObject = NewObject<UOneSkyLocalizationTargetSettings>();
	}

	/** Get the OneSky connection name */
	const FString& GetConnectionName() const;

	/** Load settings from ini file */
	void LoadSettings();

	/** Save settings to ini file */
	void SaveSettings() const;

	/** Get the credentials we use to access the server - only call on the game thread */
	FOneSkyConnectionInfo GetConnectionInfo() const;

	/** Set the public API key we use to access the server - only call on the game thread */
	void SetApiKey(FString InApiKey)
	{
		check(IsInGameThread());
		ConnectionInfo.ApiKey = InApiKey;
	}
	
	/** Set the secret API key we use to access the server - only call on the game thread */
	void SetApiSecret(FString InApiSecret)
	{
		check(IsInGameThread());
		ConnectionInfo.ApiSecret = InApiSecret;
	}

	/** Set whether or not to save the secret API key (WARNING: saved unencrypted) */
	void SetSaveSecretKey(bool InSaveSecretKey)
	{
		check(IsInGameThread());
		bSaveSecretKey = InSaveSecretKey;
	}

	/** Get whether or not to save the secret API Key */
	bool GetSaveSecretKey()
	{
		check(IsInGameThread());
		return bSaveSecretKey;
	}

	/** Get settings for a specific localization target by it's Guid
	*	@param TargetGuid - the GUID to search for
	*	@param bCreateIfNotFound - Add a new blank settings entry to the array if not found
	*/
	FOneSkyLocalizationTargetSetting* GetSettingsForTarget(FGuid TargetGuid, bool bCreateIfNotFound = false);

	/** Set settings for a specific localization target by it's Guid */
	void SetSettingsForTarget(FGuid TargetGuid, int32 ProjectId, FString FileName);

public:
	/** Object that contains array of OneSky settings for a given targets, used to serialize to .ini file */
	UOneSkyLocalizationTargetSettings* TargetSettingsObject;

private:
	/** A critical section for settings access */
	mutable FCriticalSection CriticalSection;

	/** The credentials we use to access the server */
	FOneSkyConnectionInfo ConnectionInfo;

	/** Whether or not we remember the secret key for the user (WARNING: saved unencrypted) */
	bool bSaveSecretKey;
}; 
