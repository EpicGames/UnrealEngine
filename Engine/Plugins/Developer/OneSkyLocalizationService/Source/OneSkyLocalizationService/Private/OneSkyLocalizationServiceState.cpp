// Copyright Epic Games, Inc. All Rights Reserved.

#include "OneSkyLocalizationServiceState.h"
#include "Internationalization/Internationalization.h"
#include "OneSkyLocalizationServiceRevision.h"

#define LOCTEXT_NAMESPACE "OneSkyLocalizationService.State"

int32 FOneSkyLocalizationServiceState::GetHistorySize() const
{
	return History.Num();
}

TSharedPtr<class ILocalizationServiceRevision, ESPMode::ThreadSafe> FOneSkyLocalizationServiceState::GetHistoryItem( int32 HistoryIndex ) const
{
	check(History.IsValidIndex(HistoryIndex));
	return History[HistoryIndex];
}

FName FOneSkyLocalizationServiceState::GetIconName() const
{
	if (!IsCurrent())
	{
		return FName("OneSky.NotAtHeadRevision");
	}

	switch (State)
	{
	default:
	case EOneSkyState::Unknown:
		return NAME_None;
	case EOneSkyState::Untranslated:
		return FName("OneSky.Untranslated");
	case EOneSkyState::NotAccepted:
		return FName("OneSky.NotAccepted");
	case EOneSkyState::NotFinalized:
		return FName("OneSky.NotFinalized");
	case EOneSkyState::Finalized:
		return FName("OneSky.Finalized");
	case EOneSkyState::Deprecated:
		return FName("OneSky.Deprecated");
	}
}

FName FOneSkyLocalizationServiceState::GetSmallIconName() const
{
	if (!IsCurrent())
	{
		return FName("OneSky.NotAtHeadRevision_Small");
	}

	switch (State)
	{
	default:
	case EOneSkyState::Unknown:
		return NAME_None;
	case EOneSkyState::Untranslated:
		return FName("OneSky.Untranslated_Small");
	case EOneSkyState::NotAccepted:
		return FName("OneSky.NotAccepted_Small");
	case EOneSkyState::NotFinalized:
		return FName("OneSky.NotFinalized_Small");
	case EOneSkyState::Finalized:
		return FName("OneSky.Finalized_Small");
	case EOneSkyState::Deprecated:
		return FName("OneSky.Deprecated_Small");
	}
}

FText FOneSkyLocalizationServiceState::GetDisplayName() const
{

	if( !IsCurrent() )
	{
		return LOCTEXT("NotCurrent", "Not current");
	}

	switch(State)
	{
	default:
	case EOneSkyState::Unknown:
		return LOCTEXT("Unknown", "Unknown to OneSky");
	case EOneSkyState::Untranslated:
		return LOCTEXT("Untranslated", "Untranslated in OneSky");
	case EOneSkyState::NotAccepted:
		return LOCTEXT("NotAccepted", "Translation not accepted in OneSky");
	case EOneSkyState::NotFinalized:
		return LOCTEXT("NotFinalized", "Translation accepted in OneSky");
	case EOneSkyState::Finalized:
		return LOCTEXT("Finalized", "Translation finalized in OneSky");
	case EOneSkyState::Deprecated:
		return LOCTEXT("Deprecated", "Translation deprecated in OneSky");
	}
}

FText FOneSkyLocalizationServiceState::GetDisplayTooltip() const
{
	if( !IsCurrent() )
	{
		return LOCTEXT("NotCurrent_Tooltip", "The text(s) are not at the latest revision");
	}

	switch(State)
	{
	default:
	case EOneSkyState::Unknown:
		return LOCTEXT("Unknown_Tooltip", "The text(s) status is unknown to OneSky");
	case EOneSkyState::Untranslated:
		return LOCTEXT("Untranslated_Tooltip", "The text(s) is known to OneSky, but there is no translation for the given culture.");
	case EOneSkyState::NotAccepted:
		return LOCTEXT("NotAccepted_Tooltip", "The text(s) is know to OneSky and there is a translation for the given culture but it is not accepted.");
	case EOneSkyState::NotFinalized:
		return LOCTEXT("NotFinalized_Tooltip", "The text(s) is know to OneSky, and a translation for the given culture has been accepted, but not finalized.");
	case EOneSkyState::Finalized:
		return LOCTEXT("Finalized_Tooltip", "The text(s) is know to OneSky, and a translation for the given culture has been accepted and finalized.");
	case EOneSkyState::Deprecated:
		return LOCTEXT("Deprecated_Tooltip", "The text(s) is know to OneSky, and a translation existed previously but it has been deprecated.");
	};
}

const FString& FOneSkyLocalizationServiceState::GetSourceString() const
{
	return TranslationId.Source;
}

const FCulturePtr FOneSkyLocalizationServiceState::GetCulture() const
{
	return TranslationId.Culture;
}


const FDateTime& FOneSkyLocalizationServiceState::GetTimeStamp() const
{
	return TimeStamp;
}

bool FOneSkyLocalizationServiceState::IsCurrent() const
{
	return LocalRevNumber == OneSkyLatestRevNumber;
}

bool FOneSkyLocalizationServiceState::IsKnownToLocalizationService() const
{
	return State != EOneSkyState::Unknown;
}

bool FOneSkyLocalizationServiceState::CanEdit() const
{
	// TODO: Check if this user can add/accept translations, or change finalized state
	return true;
}

bool FOneSkyLocalizationServiceState::IsUnknown() const
{
	return State == EOneSkyState::Unknown;
}

bool FOneSkyLocalizationServiceState::IsModified() const
{
	return bModifed;
}

#undef LOCTEXT_NAMESPACE
