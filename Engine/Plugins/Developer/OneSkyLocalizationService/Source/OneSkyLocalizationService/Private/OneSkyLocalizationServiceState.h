// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILocalizationServiceState.h"
#include "Misc/DateTime.h"

class FOneSkyLocalizationServiceRevision;

namespace EOneSkyState
{
	enum Type
	{
		/** Text unknown to OneSky */
		Unknown = 0,

		/** OneSky has an entry for this text, but no translation for this culture */
		Untranslated = 1,

		/** One sky has an entry for this text and a translation, but the translation has not been accepted*/
		NotAccepted = 2,

		/** One sky has an entry for this text and an accepted translation, but the translation is not finalized*/
		NotFinalized = 3,

		/** One sky has an entry for this text and a finalized translation */
		Finalized = 4,

		/** One sky has an entry for this text, and a translation, but that translation is deprecated */
		Deprecated = 5,
	};
}

class FOneSkyLocalizationServiceState : public ILocalizationServiceState
{
public:
	FOneSkyLocalizationServiceState(const FLocalizationServiceTranslationIdentifier InTranslationId, EOneSkyState::Type InState = EOneSkyState::Unknown)
		: TranslationId(InTranslationId)
		, State(InState)
		, OneSkyLatestRevNumber(INVALID_REVISION)
		, LocalRevNumber(INVALID_REVISION)
		, bModifed(false)
		, TimeStamp(0)
	{
	}

	/** ILocalizationServiceState interface */
	virtual int32 GetHistorySize() const override;
	virtual TSharedPtr<class ILocalizationServiceRevision, ESPMode::ThreadSafe> GetHistoryItem( int32 HistoryIndex ) const override;
	virtual FName GetIconName() const override;
	virtual FName GetSmallIconName() const override;
	virtual FText GetDisplayName() const override;
	virtual FText GetDisplayTooltip() const override;
	virtual const FString& GetSourceString() const override;
	virtual const FCulturePtr GetCulture() const override;
	virtual const FDateTime& GetTimeStamp() const override;
	virtual bool IsCurrent() const override;
	virtual bool IsKnownToLocalizationService() const override;
	virtual bool IsUnknown() const override;
	virtual bool CanEdit() const override;
	virtual bool IsModified() const override;
	virtual const FLocalizationServiceTranslationIdentifier& GetTranslationIdentifier() const override
	{
		return TranslationId;
	}

	/** Get the state of a file */
	EOneSkyState::Type GetState() const
	{
		return State;
	}

	/** Set the state of the translation */
	void SetState( EOneSkyState::Type InState )
	{
		State = InState;
	}

	/** Update the translation */
	void SetTranslation (const FString& InTranslation)
	{
		Translation = InTranslation;
	}

	virtual const FString& GetTranslationString() const override
	{
		return Translation;
	}

public:
	/** History of the item, if any */
	TArray< TSharedRef<FOneSkyLocalizationServiceRevision, ESPMode::ThreadSafe> > History;

	/** Translation identifier */
	FLocalizationServiceTranslationIdentifier TranslationId;

	/** Current translation */
	FString Translation;

	/** Status of the file */
	EOneSkyState::Type State;

	/** Latest revision number of the text in OneSky */
	int OneSkyLatestRevNumber;

	/** Latest rev number at which the text was synced to before being edited */
	int LocalRevNumber;

	/** Modified from depot version */
	bool bModifed;

	/** The timestamp of the last update */
	FDateTime TimeStamp;
};
